{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Network Prediction Insights",
	"Description": "Allows debugging of NetworkPrediction via Unreal Insights",
	"Category": "Insights",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
		{
			"Name": "NetworkPredictionInsights",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default",
			"ProgramAllowList": [
				"UnrealInsights"
			]
		}
	]
}
