// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NetworkPredictionInsights : ModuleRules
	{
		public NetworkPredictionInsights(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"InputCore",
				"SlateCore",
				"Slate",
				"TraceLog",
				"TraceAnalysis",
				"TraceServices",
				"TraceInsights",
				"TraceInsightsCore",
				"AssetRegistry",
				"ApplicationCore",
			});

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"CoreUObject",
				});
			}

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorFramework",
					"UnrealEd",
					"EditorWidgets",
				});
			}
		}
	}
}

