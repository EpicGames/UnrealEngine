// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionAnalyzer.h"

#include "HAL/LowLevelMemTracker.h"
#include "NetworkPredictionProvider.h"
#include "TraceServices/Utils.h"

// As we are tracing from multiple threads and we have trace events that are stringed together we need to track some state per thread to not trace bad data.
struct FNetworkPredictionAnalyzer::FThreadState
{
	int32 TraceID = INDEX_NONE;
	int32 TickStartMS;
	int32 TickDeltaMS;
	int32 TickOutputFrame;
	int32 TickLocalOffsetFrame = 0;
	bool bLocalOffsetFrameChanged = false;
	int32 PendingWriteFrame;
	ENP_UserStateSource PendingUserStateSource = ENP_UserStateSource::Unknown;
};

FNetworkPredictionAnalyzer::FNetworkPredictionAnalyzer(TraceServices::IAnalysisSession& InSession, FNetworkPredictionProvider& InNetworkPredictionProvider)
	: Session(InSession)
	, NetworkPredictionProvider(InNetworkPredictionProvider)
{
}

void FNetworkPredictionAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;
	
	Builder.RouteEvent(RouteId_SimulationScope, "NetworkPrediction", "SimulationScope");
	Builder.RouteEvent(RouteId_SimulationCreated, "NetworkPrediction", "SimulationCreated");
	Builder.RouteEvent(RouteId_SimulationConfig, "NetworkPrediction", "SimulationConfig");

	Builder.RouteEvent(RouteId_WorldFrameStart, "NetworkPrediction", "WorldFrameStart");
	Builder.RouteEvent(RouteId_PieBegin, "NetworkPrediction", "PieBegin");
	Builder.RouteEvent(RouteId_Version, "NetworkPrediction", "Version");
	Builder.RouteEvent(RouteId_WorldPreInit, "NetworkPrediction", "WorldPreInit");
	Builder.RouteEvent(RouteId_SystemFault, "NetworkPrediction", "SystemFault");

	Builder.RouteEvent(RouteId_Tick, "NetworkPrediction", "Tick");
	Builder.RouteEvent(RouteId_SimTick, "NetworkPrediction", "SimTick");
	Builder.RouteEvent(RouteId_SimulationState, "NetworkPrediction", "SimState");

	Builder.RouteEvent(RouteId_NetRecv, "NetworkPrediction", "NetRecv");
	Builder.RouteEvent(RouteId_ShouldReconcile, "NetworkPrediction", "ShouldReconcile");
	Builder.RouteEvent(RouteId_Reconcile, "NetworkPrediction", "Reconcile");
	Builder.RouteEvent(RouteId_RollbackInject, "NetworkPrediction", "RollbackInject");
	
	Builder.RouteEvent(RouteId_PushInputFrame, "NetworkPrediction", "PushInputFrame");
	Builder.RouteEvent(RouteId_FixedTickOffset, "NetworkPrediction", "FixedTickOffset");

	Builder.RouteEvent(RouteId_ProduceInput, "NetworkPrediction", "ProduceInput");
	Builder.RouteEvent(RouteId_BufferedInput, "NetworkPrediction", "BufferedInput");

	Builder.RouteEvent(RouteId_OOBStateMod, "NetworkPrediction", "OOBStateMod");

	Builder.RouteEvent(RouteId_InputCmd, "NetworkPrediction", "InputCmd");
	Builder.RouteEvent(RouteId_SyncState, "NetworkPrediction", "SyncState");
	Builder.RouteEvent(RouteId_AuxState, "NetworkPrediction", "AuxState");
	Builder.RouteEvent(RouteId_PhysicsState, "NetworkPrediction", "PhysicsState");
}

void FNetworkPredictionAnalyzer::OnAnalysisEnd()
{
}

bool FNetworkPredictionAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FNetworkPredictionAnalyzer"));

	TraceServices::FAnalysisSessionEditScope _(Session);
	const auto& EventData = Context.EventData;

	auto ParseUserState = [&EventData, &Context, this](ENP_UserState Type)
	{
		FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

		FString Value = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentString<ANSICHAR>("Value", Context);
		NetworkPredictionProvider.WriteUserState(ThreadState.TraceID, ThreadState.PendingWriteFrame, this->EngineFrameNumber, Type, ThreadState.PendingUserStateSource, Session.StoreString(*Value));
	};

	bool bUnhandled = false;
	switch (RouteId)
	{
		case RouteId_SimulationScope:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());
			ThreadState.TraceID = EventData.GetValue<int32>("TraceID");
			break;
		}

		case RouteId_SimulationState:
		{
			// This event is issues when we are updating an already ticked simulation a state with new state data
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			ThreadState.TraceID = EventData.GetValue<int32>("TraceID");
			ThreadState.PendingWriteFrame = ThreadState.TickOutputFrame;
			ThreadState.PendingUserStateSource = ENP_UserStateSource::SimTick;
			break;
		}

		case RouteId_SimulationCreated:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			ThreadState.TraceID = EventData.GetValue<int32>("TraceID");

			FSimulationData::FConst& ConstData = NetworkPredictionProvider.WriteSimulationCreated(ThreadState.TraceID);
			ConstData.DebugName = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentString<ANSICHAR>("DebugName", Context);
			ConstData.ID.SimID = EventData.GetValue<uint32>("SimulationID");
			break;
		}

		case RouteId_SimulationConfig:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());
			ThreadState.TraceID = EventData.GetValue<int32>("TraceID");

			ensure(EngineFrameNumber > 0);

			NetworkPredictionProvider.WriteSimulationConfig(ThreadState.TraceID,
				EngineFrameNumber,
				(ENP_NetRole)EventData.GetValue<uint8>("NetRole"),
				(bool)EventData.GetValue<uint8>("bHasNetConnection"),
				(ENP_TickingPolicy)EventData.GetValue<uint8>("TickingPolicy"),
				(ENP_NetworkLOD)EventData.GetValue<uint8>("NetworkLOD"),
				EventData.GetValue<int32>("ServiceMask"));
			break;
		}

		case RouteId_WorldFrameStart:
		{
			EngineFrameNumber = EventData.GetValue<uint64>("EngineFrameNumber");
			DeltaTimeSeconds = EventData.GetValue<float>("DeltaTimeSeconds");
			break;
		}

		case RouteId_PieBegin:
		{
			EngineFrameNumber = EventData.GetValue<uint64>("EngineFrameNumber");
			NetworkPredictionProvider.WritePIEStart();
			break;
		}

		case RouteId_Version:
		{
			const uint32 Version = EventData.GetValue<uint32>("Version");
			NetworkPredictionProvider.SetNetworkPredictionTraceVersion(Version);
			break;
		}

		case RouteId_WorldPreInit:
		{
			EngineFrameNumber = EventData.GetValue<uint64>("EngineFrameNumber");
			break;
		}

		case RouteId_SystemFault:
		{
			ensure(EngineFrameNumber > 0);

			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			FString Message = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentString<WIDECHAR>("Message", Context);
			const TCHAR* StoredString = Session.StoreString(*Message);
			ensureMsgf(ThreadState.TraceID > 0, TEXT("Invalid TraceID when analyzing SystemFault: %s"), StoredString);

			NetworkPredictionProvider.WriteSystemFault(
				ThreadState.TraceID,
				EngineFrameNumber,
				StoredString);
			break;
		}

		case RouteId_Tick:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			ThreadState.TickStartMS = EventData.GetValue<int32>("StartMS");
			ThreadState.TickDeltaMS = EventData.GetValue<int32>("DeltaMS");
			ThreadState.TickOutputFrame = EventData.GetValue<int32>("OutputFrame");
			break;
		}

		case RouteId_SimTick:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			ThreadState.TraceID = EventData.GetValue<int32>("TraceID");

			FSimulationData::FTick TickData;
			TickData.EngineFrame = EngineFrameNumber;
			TickData.StartMS = ThreadState.TickStartMS;
			TickData.EndMS = ThreadState.TickStartMS + ThreadState.TickDeltaMS;
			TickData.OutputFrame = ThreadState.TickOutputFrame;
			TickData.LocalOffsetFrame = ThreadState.TickLocalOffsetFrame;

			NetworkPredictionProvider.WriteSimulationTick(ThreadState.TraceID, MoveTemp(TickData));

			ThreadState.PendingWriteFrame = ThreadState.TickOutputFrame;
			ThreadState.PendingUserStateSource = ENP_UserStateSource::SimTick;

			ensure(ThreadState.PendingWriteFrame >= 0);
			break;
		}

		case RouteId_InputCmd:
		{
			ParseUserState(ENP_UserState::Input);
			break;
		}
		case RouteId_SyncState:
		{
			ParseUserState(ENP_UserState::Sync);
			break;
		}
		case RouteId_AuxState:
		{
			ParseUserState(ENP_UserState::Aux);
			break;
		}

		case RouteId_PhysicsState:
		{
			ParseUserState(ENP_UserState::Physics);
			break;
		}

		case RouteId_NetRecv:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());			
			ensure(EngineFrameNumber > 0);
			ensure(ThreadState.TraceID > 0);

			ThreadState.PendingUserStateSource = ENP_UserStateSource::NetRecv;
			ThreadState.PendingWriteFrame = EventData.GetValue<int32>("Frame");
			ensure(ThreadState.PendingWriteFrame >= 0);

			FSimulationData::FNetSerializeRecv NetRecv;
			NetRecv.EngineFrame = EngineFrameNumber;
			NetRecv.SimTimeMS = EventData.GetValue<int32>("TimeMS");
			NetRecv.Frame = ThreadState.PendingWriteFrame;

			NetworkPredictionProvider.WriteNetRecv(ThreadState.TraceID, MoveTemp(NetRecv));
			break;
		}

		case RouteId_ShouldReconcile:
		{
			const FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			// TraceID should have already been traced via RouteId_SimulationScope before the reconcile was actually evaluated
			const int32 ReconcileTraceID = EventData.GetValue<int32>("TraceID");
			ensure(ThreadState.TraceID == ReconcileTraceID);

			NetworkPredictionProvider.WriteReconcile(ThreadState.TraceID, ThreadState.TickLocalOffsetFrame, ThreadState.bLocalOffsetFrameChanged);
			break;
		}

		case RouteId_Reconcile:
		{
			const FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			// Valid TraceID should have already been set
			ensure(ThreadState.TraceID > 0);

			FString UserString = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentString<ANSICHAR>("UserString", Context);
			NetworkPredictionProvider.WriteReconcileStr(ThreadState.TraceID, Session.StoreString(*UserString));
			break;
		}

		case RouteId_RollbackInject:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			// This isn't accurate anymore. We should remove "NetCommit" from the provider side and distinguish between "caused a rollback" and "participated in a rollback"
			// (E.g, ShouldReconcile vs RollbackObject)
			const int32 RollbackInjectTraceID = EventData.GetValue<int32>("TraceID");
			ThreadState.TraceID = RollbackInjectTraceID;
			NetworkPredictionProvider.WriteNetCommit(RollbackInjectTraceID);
			break;
		}

		case RouteId_PushInputFrame:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			ThreadState.PendingWriteFrame = EventData.GetValue<int32>("Frame");
			ensure(ThreadState.PendingWriteFrame >= 0);

			break;
		}

		case RouteId_FixedTickOffset:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			ThreadState.TickLocalOffsetFrame = EventData.GetValue<int32>("Offset");
			ThreadState.bLocalOffsetFrameChanged = EventData.GetValue<bool>("Changed");
			break;
		}

		case RouteId_BufferedInput:
		{
			const FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());
			
			ensureMsgf(ThreadState.TraceID > 0, TEXT("Invalid TraceID when analyzing BufferedInput."));
			NetworkPredictionProvider.WriteBufferedInput(ThreadState.TraceID, EventData.GetValue<int32>("NumBufferedFrames"), EventData.GetValue<bool>("bFault"));

			break;
		}

		case RouteId_ProduceInput:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());
			ThreadState.TraceID = EventData.GetValue<int32>("TraceID");
			ThreadState.PendingUserStateSource = ENP_UserStateSource::ProduceInput;
			NetworkPredictionProvider.WriteProduceInput(ThreadState.TraceID);
			break;
		}

		case RouteId_OOBStateMod:
		{
			FThreadState& ThreadState = GetThreadState(Context.ThreadInfo.GetId());

			ThreadState.TraceID = EventData.GetValue<int32>("TraceID");
			ThreadState.PendingWriteFrame = EventData.GetValue<int32>("Frame");
			ensure(ThreadState.PendingWriteFrame >= 0);

			ThreadState.PendingUserStateSource = ENP_UserStateSource::OOB;
			NetworkPredictionProvider.WriteOOBStateMod(ThreadState.TraceID);

			FString Source = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentString<ANSICHAR>("Source", Context);
			NetworkPredictionProvider.WriteOOBStateModStr(ThreadState.TraceID, Session.StoreString(Source));
			break;
		}
	}

	if (!bUnhandled)
	{
		NetworkPredictionProvider.IncrementDataCounter();
	}

	return true;
}

FNetworkPredictionAnalyzer::FThreadState& FNetworkPredictionAnalyzer::GetThreadState(uint32 ThreadId)
{
	FThreadState* ThreadState = ThreadStatesMap.FindRef(ThreadId);
	if (!ThreadState)
	{
		ThreadState = new FThreadState();
		ThreadStatesMap.Add(ThreadId, ThreadState);
	}
	return *ThreadState;
}

