// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Containers/Map.h"

class FNetworkPredictionProvider;

namespace TraceServices { class IAnalysisSession; }

// Analyzes events that are contained in a trace,
// Works by subscribing to events by name along with user-provided "route" identifiers
// To analyze a trace, concrete IAnalyzer-derived objects are registered with a
// FAnalysisContext which is then asked to launch and coordinate the analysis.
//
// The analyzer is what populates the data in the Provider.

class FNetworkPredictionAnalyzer : public UE::Trace::IAnalyzer
{
public:

	FNetworkPredictionAnalyzer(TraceServices::IAnalysisSession& InSession, FNetworkPredictionProvider& InNetworkPredictionProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:

	enum : uint16
	{
		RouteId_SimulationScope,
		RouteId_SimulationState,
		RouteId_SimulationCreated,
		RouteId_SimulationConfig,
		RouteId_WorldFrameStart,
		RouteId_Version,
		RouteId_WorldPreInit,
		RouteId_PieBegin,
		RouteId_SystemFault,
		RouteId_Tick,
		RouteId_SimTick,
		RouteId_InputCmd,
		RouteId_SyncState,
		RouteId_AuxState,
		RouteId_PhysicsState,
		RouteId_NetRecv,
		RouteId_ShouldReconcile,
		RouteId_Reconcile,
		RouteId_RollbackInject,
		RouteId_PushInputFrame,
		RouteId_FixedTickOffset,
		RouteId_ProduceInput,
		RouteId_BufferedInput,
		RouteId_OOBStateMod
	};


	TraceServices::IAnalysisSession& Session;
	FNetworkPredictionProvider& NetworkPredictionProvider;

	// WorldFrame, always from main thread
	uint64 EngineFrameNumber=0;
	float DeltaTimeSeconds;

	// As we are tracing from multiple threads and we have trace events that are stringed together we need to track some state per thread
	struct FThreadState;

	TMap<uint32, FThreadState*> ThreadStatesMap;
	FThreadState& GetThreadState(uint32 ThreadId);
};
