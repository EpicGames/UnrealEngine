// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionInsightsModule.h"

#include "Features/IModularFeatures.h"
#include "INetworkPredictionInsightsModule.h" // IWYU pragma: keep
#include "Insights/IUnrealInsightsModule.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "String/ParseTokens.h"
#include "Misc/Paths.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "UI/NetworkPredictionInsightsManager.h"
#include "UI/SNPWindow.h"

#if WITH_ENGINE
#include "Engine/Engine.h"
#endif


#define LOCTEXT_NAMESPACE "NetworkPredictionInsightsModule"

const FName NetworkPredictionInsightsTabs::DocumentTab("DocumentTab");

const FName FNetworkPredictionInsightsModule::InsightsTabName("NetworkPrediction");

void FNetworkPredictionInsightsModule::StartupModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/NetworkPredictionInsights"));

	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &NetworkPredictionTraceModule);

	FNetworkPredictionInsightsManager::Initialize();
	IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

	bool bShouldStartNetworkTrace = false;

	FString EnabledChannels;
	FParse::Value(FCommandLine::Get(), TEXT("-trace="), EnabledChannels, false);
	UE::String::ParseTokens(EnabledChannels, TEXT(","), [&bShouldStartNetworkTrace](FStringView Token) {
		if (Token.Compare(TEXT("NetworkPrediction"), ESearchCase::IgnoreCase) == 0 || Token.Compare(TEXT("NP"), ESearchCase::IgnoreCase) == 0)
		{
			bShouldStartNetworkTrace = true;
		}
		});

	// Auto spawn the Network Prediction Insights tab if we detect NP data
	// Only do this in Standalone UnrealInsights.exe. In Editor the user will select the NPI window manually.
	// (There is currently not way to extend the high level layout of unreal insights, e.g, the layout created in FTraceInsightsModule::CreateSessionBrowser)
	// (only SetUnrealInsightsLayoutIni which is extending the layouts of pre made individual tabs, not the the overall session layout)
	if (!GIsEditor)
	{
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("NetworkPredictionInsights"), 0.0f, [&UnrealInsightsModule](float DeltaTime)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FNetworkPredictionInsightsModule_Tick);
			auto SessionPtr = UnrealInsightsModule.GetAnalysisSession();
			if (SessionPtr.IsValid())
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*SessionPtr.Get());
				if (const INetworkPredictionProvider* NetworkPredictionProvider = ReadNetworkPredictionProvider(*SessionPtr.Get()))
				{
					auto NetworkPredictionTraceVersion = NetworkPredictionProvider->GetNetworkPredictionTraceVersion();

					if (NetworkPredictionTraceVersion > 0)
					{
						static bool HasSpawnedTab = false;
						if (!HasSpawnedTab && FGlobalTabmanager::Get()->HasTabSpawner(FNetworkPredictionInsightsModule::InsightsTabName))
						{
							HasSpawnedTab = true;
							FGlobalTabmanager::Get()->TryInvokeTab(FNetworkPredictionInsightsModule::InsightsTabName);
						}
					}
				}
			}

			return true;
		});
	}

	// Actually register our tab spawner
	FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FNetworkPredictionInsightsModule::InsightsTabName,
		FOnSpawnTab::CreateLambda([bShouldStartNetworkTrace](const FSpawnTabArgs& Args)
	{
		LLM_SCOPE_BYNAME(TEXT("Insights/NetworkPredictionInsights"));

		if (bShouldStartNetworkTrace)
		{
			StartNetworkTrace();
		}

		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab);

		TSharedRef<SNPWindow> Window = SNew(SNPWindow, DockTab, Args.GetOwnerWindow());
		DockTab->SetContent(Window);
		return DockTab;
	}))
		.SetDisplayName(NSLOCTEXT("FNetworkPredictionInsightsModule", "NetworkPredictionTabTitle", "Network Prediction Insights"))
		.SetTooltipText(NSLOCTEXT("FNetworkPredictionInsightsModule", "FilteringTabTooltip", "Opens the Network Prediction Insights tab."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProfilerCommand.StatsProfiler.Small"));

	//TabSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsProfilingCategory());
	TabSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());

	// -------------------------------------------------------------

#if WITH_EDITOR
	if (!IsRunningCommandlet() && bShouldStartNetworkTrace)
	{
		// Conditionally create local store service after engine init (if someone doesn't beat us to it).
		// This is temp until a more formal local server is done by the insights system.
		StoreServiceHandle = FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([this]
		{
			LLM_SCOPE_BYNAME(TEXT("Insights/NetworkPredictionInsights"));
			IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

			if (!UnrealInsightsModule.GetStoreClient())
			{
				UE_LOGF(LogCore, Display, "NetworkPredictionInsights module auto-connecting to local trace server...");
				UnrealInsightsModule.ConnectToStore(TEXT("127.0.0.1"));
				UnrealInsightsModule.CreateSessionViewer(false);
			}
		});
	}
#endif // WITH_EDITOR
}

void FNetworkPredictionInsightsModule::ShutdownModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/NetworkPredictionInsights"));

	if (StoreServiceHandle.IsValid())
	{
		FCoreDelegates::OnFEngineLoopInitComplete.Remove(StoreServiceHandle);
	}

	FTSTicker::RemoveTicker(TickerHandle);

	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &NetworkPredictionTraceModule);
}

void FNetworkPredictionInsightsModule::StartNetworkTrace()
{
#if WITH_EDITOR
	UE::Trace::ToggleChannel(TEXT("NetworkPredictionChannel"), true);

	const bool bConnected = FTraceAuxiliary::Start(
		FTraceAuxiliary::EConnectionType::Network,
		TEXT("127.0.0.1"),
		nullptr);

	IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	UnrealInsightsModule.StartAnalysisForLastLiveSession();
#endif
}

IMPLEMENT_MODULE(FNetworkPredictionInsightsModule, NetworkPredictionInsights);

#undef LOCTEXT_NAMESPACE
