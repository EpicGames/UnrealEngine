// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Modules/ModuleInterface.h"
#include "NetworkPredictionTraceModule.h"
#include "Trace/StoreService.h"

struct FInsightsMajorTabExtender;

namespace UE
{
namespace Trace 
{
	class FStoreClient;
}
}

class FNetworkPredictionInsightsModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	static void StartNetworkTrace();

	FNetworkPredictionTraceModule NetworkPredictionTraceModule;

	FTSTicker::FDelegateHandle TickerHandle;
	FDelegateHandle StoreServiceHandle;

	//TWeakPtr<FTabManager> WeakTimingProfilerTabManager;

	static const FName InsightsTabName;	
};
