// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - TODO

/*

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

class FNetworkPredictionInsightsStyle
	: public FSlateStyleSet
{
public:
	FNetworkPredictionInsightsStyle()
		: FSlateStyleSet("NetworkPredictionInsightsStyle")
		, SelectionColor_LinearRef(MakeShareable(new FLinearColor( 0.728f, 0.364f, 0.003f)))
		, SelectionColor_Pressed_LinearRef(MakeShareable(new FLinearColor( 0.701f, 0.225f, 0.003f)))
		, SelectionColor(SelectionColor_LinearRef)
		, SelectionColor_Pressed(SelectionColor_Pressed_LinearRef)
	{
		const FVector2D Icon10x10(10.0f, 10.0f);
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon24x24(24.0f, 24.0f);
		const FVector2D Icon32x32(32.0f, 32.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Animation/NetworkPredictionInsights/Content"));

		const FSlateColor DefaultForeground(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));

		Set("SchematicViewRootLeft", new BOX_BRUSH("SchematicViewRootLeft", FMargin(4/16.0f, 8.0f/16.0f, 4/16.0f, 4/16.0f)));
		Set("SchematicViewRootMid", new BOX_BRUSH("SchematicViewRootMid", FMargin(4/16.0f, 8.0f/16.0f, 4/16.0f, 4/16.0f)));

		Set("SchematicViewViewButton", FButtonStyle()
			.SetNormal(FSlateNoResource())
			.SetHovered(BOX_BRUSH("RoundedSelection_16x", 4.0f/16.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed))
			.SetNormalPadding(FMargin(2, 2, 2, 2))
			.SetPressedPadding(FMargin(2, 3, 2, 1))
		);

		Set("SchematicViewViewButtonIcon", new IMAGE_BRUSH("view_button", Icon20x20));

		Set("TransportControls.HyperlinkSpinBox", FSpinBoxStyle()
			.SetTextPadding(FMargin(0))
			.SetBackgroundBrush(BORDER_BRUSH("HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), FSlateColor::UseSubduedForeground()))
			.SetHoveredBackgroundBrush(FSlateNoResource())
			.SetInactiveFillBrush(FSlateNoResource())
			.SetActiveFillBrush(FSlateNoResource())
			.SetForegroundColor(FSlateColor::UseSubduedForeground())
			.SetArrowsImage(FSlateNoResource())
		);

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FNetworkPredictionInsightsStyle& Get()
	{
		static FNetworkPredictionInsightsStyle Inst;
		return Inst;
	}
	
	~FNetworkPredictionInsightsStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	const TSharedRef<FLinearColor> SelectionColor_LinearRef;
	const TSharedRef<FLinearColor> SelectionColor_Pressed_LinearRef;
	const FSlateColor SelectionColor;
	const FSlateColor SelectionColor_Pressed;

	const TSharedRef<FLinearColor> SimFrameColor_Confirmed_Ref;
	const TSharedRef<FLinearColor> SimFrameColor_Corrected_Ref;
	const TSharedRef<FLinearColor> SimFrameColor_Predicted_Ref;
	const FSlateColor SimFrameColor_Confirmed;
	const FSlateColor SimFrameColor_Corrected;
	const FSlateColor SimFrameColor_Predicted;

	//0.53333  0.67059  0.53333
};

*/
