// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionProvider.h"

FName FNetworkPredictionProvider::ProviderName("NetworkPredictionProvider");

DEFINE_LOG_CATEGORY_STATIC(LogNetworkPredictionTraceProvider, Log, All);

// -----------------------------------------------------------------------------

FNetworkPredictionProvider::FNetworkPredictionProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{

}

void FNetworkPredictionProvider::SetNetworkPredictionTraceVersion(uint32 Version)
{
	Session.WriteAccessCheck();
	NetworkPredictionTraceVersion = Version;
}

// -----------------------------------------------------------------------------

FSimulationData::FConst& FNetworkPredictionProvider::WriteSimulationCreated(int32 TraceID)
{
	FSimulationData::FConst& SimulationConstData = FindOrAdd(TraceID)->ConstData;
	SimulationConstData.ID.PIESession = PIESessionCounter;
	return SimulationConstData;
}

void FNetworkPredictionProvider::WriteSimulationTick(int32 TraceID, FSimulationData::FTick&& InTick)
{
	FSimulationData& SimulationData = FindChecked(TraceID).Get();

	SimulationData.Analysis.PendingCommitUserStates.Reset();

	// ---------------------------------------------------------------------------------------------
	//	New tick may cause previous ticks to be trashed.
	//	FIXME: Dumb approach: loop through whole list. Can be cached off 
	//	note: don't do this after adding new tick to list since we'll end up trashing ourselves
	// ---------------------------------------------------------------------------------------------
	if (SimulationData.Ticks.Num() > 0)
	{
		for (auto It = SimulationData.Ticks.GetIterator(); It; ++It)
		{
			FSimulationData::FTick& Tick = const_cast<FSimulationData::FTick&>(*It);
			if (InTick.StartMS < Tick.EndMS && Tick.TrashedEngineFrame == 0)
			{
				Tick.TrashedEngineFrame = InTick.EngineFrame;
			}
		}
	}

	// Add it to the list
	FSimulationData::FTick& NewTick = SimulationData.Ticks.PushBack();
	NewTick = MoveTemp(InTick);

	NewTick.NumBufferedInputCmds = SimulationData.Analysis.NumBufferedInputCmds;
	NewTick.bInputFault = SimulationData.Analysis.bInputFault;
	NewTick.ReconcileStr = SimulationData.Analysis.PendingReconcileStr;
	NewTick.bReconcileDueToOffsetChange = SimulationData.Analysis.bLocalFrameOffsetChanged;
	NewTick.LocalOffsetFrame = SimulationData.Analysis.LocalFrameOffset;
	SimulationData.Analysis.PendingReconcileStr = nullptr;
	SimulationData.Analysis.bLocalFrameOffsetChanged = false;

	// Repredict if we've already simulated past this time before
	if (NewTick.StartMS < SimulationData.Analysis.MaxTickSimTimeMS)
	{
		NewTick.bRepredict = true;
	}
	SimulationData.Analysis.MaxTickSimTimeMS = FMath::Max(SimulationData.Analysis.MaxTickSimTimeMS, NewTick.EndMS);

	// ---------------------------------------------------------------------------------------------
	//	Link this new tick with pending NetRecvs if possible
	// ---------------------------------------------------------------------------------------------

	TArray<FSimulationData::FNetSerializeRecv*> WorkingPendingNetSerializeRecv = MoveTemp(SimulationData.Analysis.PendingNetSerializeRecv);
	for (FSimulationData::FNetSerializeRecv* PendingRecv : WorkingPendingNetSerializeRecv)
	{
		check(PendingRecv);

		const FSimTime RecvSimTimeMS = PendingRecv->SimTimeMS;

		const bool bIsAuthority = (SimulationData.SparseData.Read(PendingRecv->EngineFrame)->NetRole == ENP_NetRole::Authority);

		// On non authority, this becomes a confirmed frame now if it wasn't already marked
		if (!bIsAuthority)
		{
			if (PendingRecv->Status == ENetSerializeRecvStatus::Unknown)
			{
				PendingRecv->Status = ENetSerializeRecvStatus::Confirm;
			}
		}

		// Perfect Match
		if (NewTick.StartMS == RecvSimTimeMS)
		{
			NewTick.StartNetRecv = PendingRecv;
			PendingRecv->NextTick = &NewTick;
		}
		// Ahead of last NetRecv
		else if (NewTick.StartMS > RecvSimTimeMS)
		{
			// Instead of attaching to the Orphaned list, find a tick to attach it to. This is a bit dicey and may not be
			// a good solution if we want to treat recvs that came in mid local predicted frame as confirmed (e.g, for sim proxies)
			auto FindNetRecvAHome = [&]()
			{
				for (auto It = SimulationData.Ticks.GetIteratorFromItem(SimulationData.Ticks.Num()-1); It; --It)
				{
					FSimulationData::FTick& PrevTick = const_cast<FSimulationData::FTick&>(*It);

					if (!PrevTick.StartNetRecv && PrevTick.StartMS == PendingRecv->SimTimeMS)
					{
						PrevTick.StartNetRecv = PendingRecv;
						PendingRecv->NextTick = &PrevTick;
						return true;
					}
				}
				return false;
			};

			if (!FindNetRecvAHome())
			{
				// No home for this recv so put it on the orphan list so it still gets drawn
				if (PendingRecv->Status == ENetSerializeRecvStatus::Unknown)
				{
					PendingRecv->Status = ENetSerializeRecvStatus::Stale;
				}
			}
		}
		else
		{
			SimulationData.Analysis.PendingNetSerializeRecv.Add(PendingRecv);
		}
	}
}

void FNetworkPredictionProvider::WriteNetRecv(int32 TraceID, FSimulationData::FNetSerializeRecv&& InNetRecv)
{
	ensure(TraceID > 0);
	FSimulationData& SimulationData = FindChecked(TraceID).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	SimulationData.Analysis.PendingCommitUserStates.Reset();

	FSimulationData::FNetSerializeRecv &NewNetSerializeRecv = SimulationData.NetRecv.PushBack();
	NewNetSerializeRecv = MoveTemp(InNetRecv);

	if (SimulationData.SparseData.Read(NewNetSerializeRecv.EngineFrame)->NetRole == ENP_NetRole::Authority)
	{
		// Authority gets net receives before running ticks, so we accumulate all of them
		SimulationData.Analysis.PendingNetSerializeRecv.Add(&NewNetSerializeRecv);
	}
	else
	{
		if (SimulationData.Analysis.PendingNetSerializeRecv.Num() > 0)
		{
			ensure(SimulationData.Analysis.PendingNetSerializeRecv.Num() == 1);

			if (SimulationData.Analysis.PendingNetSerializeRecv.Last()->Status == ENetSerializeRecvStatus::Unknown)
			{
				SimulationData.Analysis.PendingNetSerializeRecv.Last()->Status = ENetSerializeRecvStatus::Stale;
			}
			SimulationData.Analysis.PendingNetSerializeRecv.Reset();
		}
		SimulationData.Analysis.PendingNetSerializeRecv.Add(&NewNetSerializeRecv);

		if (NewNetSerializeRecv.SimTimeMS > SimulationData.Analysis.MaxTickSimTimeMS)
		{
			NewNetSerializeRecv.Status = ENetSerializeRecvStatus::Jump;
		}
	}

	// Mark ConfirmedEngineFrame on pending frame
	// We want Tick.ConfirmedEngineFrame to be the earliest engine frame we received that was timestamped > what the tick was
	// NetRecvs will always increase in sim time, but our ticks do not
	// So we need to search the remainder of the list, knowing that tick time will be not be linear
	if (SimulationData.Ticks.Num() > SimulationData.Analysis.NetRecvItemIdx)
	{
		bool bIncrementSavedIdx = true;
		for (auto It = SimulationData.Ticks.GetIteratorFromItem(SimulationData.Analysis.NetRecvItemIdx); It; ++It)
		{
			FSimulationData::FTick& Tick = const_cast<FSimulationData::FTick&>(*It);
			if (Tick.EndMS > NewNetSerializeRecv.SimTimeMS)
			{
				// Once we encounter a frame that is still unconfirmed, we can no longer increment our saved position,
				// We need to pick up back here when we get a newer NetRecv.
				bIncrementSavedIdx = false;
			}
			else if (Tick.ConfirmedEngineFrame == 0)
			{
				// This is the first NetRecv that this frame was confirmed on
				Tick.ConfirmedEngineFrame = NewNetSerializeRecv.EngineFrame;
			}

			if (bIncrementSavedIdx)
			{
				SimulationData.Analysis.NetRecvItemIdx++;
			}
		}
	}
}

void FNetworkPredictionProvider::WriteNetCommit(uint32 SimulationId)
{
	FSimulationData& SimulationData = FindChecked(SimulationId).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	// Mark pending user states committed
	for (FSimulationData::FUserState* UserState : SimulationData.Analysis.PendingCommitUserStates)
	{
		if (ensure(UserState->Source == ENP_UserStateSource::NetRecv))
		{
			UserState->Source = ENP_UserStateSource::NetRecvCommit;
		}
	}
	SimulationData.Analysis.PendingCommitUserStates.Reset();

	// Pending NetReceives also become committed here
	for (FSimulationData::FNetSerializeRecv* PendingRecv : SimulationData.Analysis.PendingNetSerializeRecv)
	{
		check(PendingRecv);

		const FSimTime RecvSimTimeMS = PendingRecv->SimTimeMS;
		const bool bIsAuthority = (SimulationData.SparseData.Read(PendingRecv->EngineFrame)->NetRole == ENP_NetRole::Authority);

		if (PendingRecv->Status == ENetSerializeRecvStatus::Unknown)
		{
			if (bIsAuthority)
			{
				// Authority this is "confirmed" (misnomer): we wait for net recvs to advance the sim (unless synth)
				PendingRecv->Status = ENetSerializeRecvStatus::Confirm;
			}
			else
			{
				// Non authority, this caused a rollback/correction since we committed to our buffers
				PendingRecv->Status = ENetSerializeRecvStatus::Rollback;
			}
		}
	}
}

void FNetworkPredictionProvider::WriteSystemFault(uint32 SimulationId, uint64 EngineFrameNumber, const TCHAR* Fmt)
{
	FSimulationData& SimulationData = FindChecked(SimulationId).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);
	
	// This is akwward to trace. Should refactor some of this so it can be easily known if this happened within a
	// NetRecv or SimTick scope.
	
	for (FSimulationData::FNetSerializeRecv* PendingRecv : SimulationData.Analysis.PendingNetSerializeRecv)
	{
		PendingRecv->Status = ENetSerializeRecvStatus::Fault;
		PendingRecv->SystemFaults.Add({Fmt});
	}

	SimulationData.Analysis.PendingSystemFaults.Add({Fmt});
}

void FNetworkPredictionProvider::WriteOOBStateMod(uint32 SimulationId)
{
	// Signals that the next user states traced will be OOB mods
	FSimulationData& SimulationData = FindChecked(SimulationId).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	SimulationData.Analysis.PendingCommitUserStates.Reset();
}

void FNetworkPredictionProvider::WriteOOBStateModStr(uint32 SimulationId, const TCHAR* Fmt)
{
	FSimulationData& SimulationData = FindChecked(SimulationId).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	SimulationData.Analysis.PendingOOBStr = Fmt;
}

void FNetworkPredictionProvider::WriteProduceInput(uint32 SimulationId)
{
	FSimulationData& SimulationData = FindChecked(SimulationId).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	SimulationData.Analysis.PendingCommitUserStates.Reset();
}

void FNetworkPredictionProvider::WriteBufferedInput(uint32 SimulationId, int32 NumBufferedInputCmds, bool bFault)
{
	FSimulationData& SimulationData = FindChecked(SimulationId).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	SimulationData.Analysis.NumBufferedInputCmds = NumBufferedInputCmds;
	SimulationData.Analysis.bInputFault = bFault;
}

void FNetworkPredictionProvider::WriteSynthInput(uint32 SimulationId)
{
	FSimulationData& SimulationData = FindChecked(SimulationId).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	SimulationData.Analysis.PendingCommitUserStates.Reset();

	// if we hook up this function the ThreadState.PendingStateSource = ENP_UserStateSource::SynthInput; should be set in the ThreadState of calling code
	check(false);
}

void FNetworkPredictionProvider::WriteSimulationConfig(int32 TraceID, uint64 EngineFrame, ENP_NetRole NetRole, bool bHasNetConnection, ENP_TickingPolicy TickingPolicy, ENP_NetworkLOD NetworkLOD, int32 ServiceMask)
{
	ensureMsgf(NetRole != ENP_NetRole::None, TEXT("NetRole was traced as None for Sim %d"), TraceID);

	FSimulationData& SimulationData = FindChecked(TraceID).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	auto& SparseData = SimulationData.SparseData.Write(EngineFrame);
	SparseData->NetRole = NetRole;
	SparseData->bHasNetConnection = bHasNetConnection;
	SparseData->TickingPolicy = TickingPolicy;
	SparseData->NetworkLOD = NetworkLOD;
	SparseData->ServiceMask = ServiceMask;
}

void FNetworkPredictionProvider::WriteUserState(int32 TraceID, int32 Frame, uint64 EngineFrame, ENP_UserState Type, ENP_UserStateSource UserStateSource, const TCHAR* UserStr)
{
	ensure(Frame >= 0);

	FSimulationData& SimulationData = FindChecked(TraceID).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	ensure(UserStateSource != ENP_UserStateSource::Unknown);

	FSimulationData::FUserState& NewUserState = SimulationData.UserData.Store[(int32)Type].Push(Frame, EngineFrame);
	NewUserState.UserStr = UserStr;
	NewUserState.Source = UserStateSource;

	if (UserStateSource == ENP_UserStateSource::NetRecv)
	{
		SimulationData.Analysis.PendingCommitUserStates.Add(&NewUserState);
	}
	else if (UserStateSource == ENP_UserStateSource::OOB)
	{
		NewUserState.OOBStr = SimulationData.Analysis.PendingOOBStr;
		SimulationData.Analysis.PendingOOBStr = nullptr;
	}
}

void FNetworkPredictionProvider::WritePIEStart()
{
	PIESessionCounter++;
}

void FNetworkPredictionProvider::WriteReconcile(int32 TraceID, int32 LocalFrameOffset, bool bLocalOffsetFrameChanged)
{
	ensure(TraceID > 0);
	FSimulationData& SimulationData = FindChecked(TraceID).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	SimulationData.Analysis.LocalFrameOffset = LocalFrameOffset;
	SimulationData.Analysis.bLocalFrameOffsetChanged = bLocalOffsetFrameChanged;
	
	if (SimulationData.Analysis.PendingReconcileStr == nullptr)
	{
		UE_LOGF(LogNetworkPredictionTraceProvider, Warning, "NP Reconcile happened without traced string. Use UE_NP_TRACE_RECONCILE");
	}
}

void FNetworkPredictionProvider::WriteReconcileStr(int32 TraceID, const TCHAR* UserStr)
{
	ensure(TraceID > 0);

	FSimulationData& SimulationData = FindChecked(TraceID).Get();
	ensureMsgf(SimulationData.ConstData.ID.PIESession >= 0, TEXT("Invalid PIE Session: %d"), SimulationData.ConstData.ID.PIESession);

	ensure(SimulationData.Analysis.PendingReconcileStr == nullptr); // Not expected to get 2+ WriteReconcileStr before WriteReconcile
	SimulationData.Analysis.PendingReconcileStr = UserStr;
}

TSharedRef<FSimulationData>& FNetworkPredictionProvider::FindOrAdd(int32 TraceID)
{
	for (TSharedRef<FSimulationData>& Data : ProviderData)
	{
		if (Data->TraceID == TraceID)
		{
			return Data;
		}
	}

	ProviderData.Add(MakeShareable(new FSimulationData(TraceID, Session.GetLinearAllocator())));
	return ProviderData.Last();
}

TSharedRef<FSimulationData>& FNetworkPredictionProvider::FindChecked(int32 TraceID)
{
	for (TSharedRef<FSimulationData>& Data : ProviderData)
	{
		if (Data->TraceID == TraceID)
		{
			return Data;
		}
	}

	checkf(false, TEXT("No matching simulation data found for TraceID: %d"), TraceID);
	return ProviderData.Last();
}

// -----------------------------------------------------------------------------
//
// -----------------------------------------------------------------------------

const INetworkPredictionProvider* ReadNetworkPredictionProvider(const TraceServices::IAnalysisSession& Session)
{
	Session.ReadAccessCheck();
	return Session.ReadProvider<INetworkPredictionProvider>(FNetworkPredictionProvider::ProviderName);
}

const TCHAR* LexToString(ENP_NetRole Role)
{
	switch (Role)
	{
	case ENP_NetRole::None:
		return TEXT("None");
	case ENP_NetRole::SimulatedProxy:
		return TEXT("Simulated");
	case ENP_NetRole::AutonomousProxy:
		return TEXT("Autonomous");
	case ENP_NetRole::Authority:
		return TEXT("Authority");	
	default:
		return TEXT("Unknown");
	}
}

const TCHAR* LexToString(ENP_UserState State)
{
	switch(State)
	{
	case ENP_UserState::Input:
		return TEXT("Input");
	case ENP_UserState::Sync:
		return TEXT("Sync");
	case ENP_UserState::Aux:
		return TEXT("Aux");
	case ENP_UserState::Physics:
		return TEXT("Physics");
	default:
		return TEXT("Unknown");
	}
}
const TCHAR* LexToString(ENP_UserStateSource Source)
{
	switch(Source)
	{
	case ENP_UserStateSource::ProduceInput:
		return TEXT("ProduceInput");
	case ENP_UserStateSource::SynthInput:
		return TEXT("SynthInput");
	case ENP_UserStateSource::SimTick:
		return TEXT("SimTick");
	case ENP_UserStateSource::NetRecv:
		return TEXT("NetRecv (Uncommited)");
	case ENP_UserStateSource::NetRecvCommit:
		return TEXT("NetRecv Committed");
	case ENP_UserStateSource::OOB:
		return TEXT("OOB");
	default:
		return TEXT("Unknown");
	}
}

const TCHAR* LexToString(ENetSerializeRecvStatus Status)
{
	switch(Status)
	{
	case ENetSerializeRecvStatus::Unknown:
		return TEXT("Unknown");
	case ENetSerializeRecvStatus::Confirm:
		return TEXT("Confirm");
	case ENetSerializeRecvStatus::Rollback:
		return TEXT("Rollback");
	case ENetSerializeRecvStatus::Jump:
		return TEXT("Jump");
	case ENetSerializeRecvStatus::Fault:
		return TEXT("Fault");
	case ENetSerializeRecvStatus::Stale:
		return TEXT("Stale");

	default:
		ensure(false);
		return TEXT("???");
	}
}

const TCHAR* LexToString(ENP_TickingPolicy Policy)
{
	switch(Policy)
	{
	case ENP_TickingPolicy::Independent:
		return TEXT("Independent");
	case ENP_TickingPolicy::Fixed:
		return TEXT("Fixed");
	default:
		ensure(false);
		return TEXT("???");
	}
}
