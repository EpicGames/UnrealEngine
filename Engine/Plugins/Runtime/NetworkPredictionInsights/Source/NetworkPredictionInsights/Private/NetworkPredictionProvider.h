// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "INetworkPredictionProvider.h"

namespace TraceServices { class IAnalysisSession; }


class FNetworkPredictionProvider : public INetworkPredictionProvider
{
public:
	static FName ProviderName;

	FNetworkPredictionProvider(TraceServices::IAnalysisSession& InSession);

	// -----------------------------------------------------
	virtual uint32 GetNetworkPredictionTraceVersion() const override { return NetworkPredictionTraceVersion; }
	virtual uint64 GetNetworkPredictionDataCounter() const { return DataCounter; }

	// -----------------------------------------------------
	
	void IncrementDataCounter() { DataCounter++; }
	void SetNetworkPredictionTraceVersion(uint32 Version);

	// ------------------------------------------------------------------------------
	//
	//	The assumption is always that we are writing *forward* in time. This will blow up 
	//	if we expect to be able to write frames in the past
	//
	// ------------------------------------------------------------------------------

	FSimulationData::FConst& WriteSimulationCreated(int32 TraceID);
	void WriteSimulationTick(int32 TraceID, FSimulationData::FTick&& InTick);
	void WriteNetRecv(int32 TraceID, FSimulationData::FNetSerializeRecv&& InNetRecv);
	void WriteNetCommit(uint32 SimulationId);
	void WriteSystemFault(uint32 SimulationId, uint64 EngineFrameNumber, const TCHAR* Fmt);
	void WriteOOBStateMod(uint32 SimulationId);
	void WriteOOBStateModStr(uint32 SimulationId, const TCHAR* Fmt);
	void WriteProduceInput(uint32 SimulationId);
	void WriteBufferedInput(uint32 SimulationId, int32 NumBufferedInputCmds, bool bFault);
	void WriteSynthInput(uint32 SimulationId);
	void WriteUserState(int32 TraceID, int32 Frame, uint64 EngineFrame, ENP_UserState Type, ENP_UserStateSource UserStateSource, const TCHAR* UserStr);
	void WritePIEStart();
	void WriteReconcileStr(int32 TraceID, const TCHAR* UserStr);
	void WriteReconcile(int32 TraceID, int32 LocalFrameOffset, bool bLocalOffsetFrameChanged);

	void WriteSimulationConfig(int32 TraceID, uint64 EngineFrame, ENP_NetRole NetRole, bool bHasNetConnection, ENP_TickingPolicy TickingPolicy, ENP_NetworkLOD NetworkLOD, int32 ServiceMask);

	virtual TArrayView<const TSharedRef<FSimulationData>> ReadSimulationData() const override
	{
		return TArrayView<const TSharedRef<FSimulationData>>(ProviderData); 
	}

private:

	TSharedRef<FSimulationData>& FindOrAdd(int32 TraceID);
	TSharedRef<FSimulationData>& FindChecked(int32 TraceID);

	// Array that stores all of our traced simulation data
	TArray<TSharedRef<FSimulationData>> ProviderData;

	uint64 DataCounter = 0;
	uint32 NetworkPredictionTraceVersion = 0;
	int32 PIESessionCounter = 0;
	TraceServices::IAnalysisSession& Session;
};
