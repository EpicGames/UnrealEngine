// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionTraceModule.h"
#include "NetworkPredictionProvider.h"
#include "NetworkPredictionAnalyzer.h"

FName FNetworkPredictionTraceModule::ModuleName("NetworkPredictionTrace");

void FNetworkPredictionTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("NetworkPrediction");
}

void FNetworkPredictionTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FNetworkPredictionProvider> NetworkPredictionProvider = MakeShared<FNetworkPredictionProvider>(InSession);
	InSession.AddProvider(FNetworkPredictionProvider::ProviderName, NetworkPredictionProvider);
	
	InSession.AddAnalyzer(new FNetworkPredictionAnalyzer(InSession, *NetworkPredictionProvider));
}

void FNetworkPredictionTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("NetworkPrediction"));
}

void FNetworkPredictionTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

