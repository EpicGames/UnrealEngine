// Copyright Epic Games, Inc. All Rights Reserved.

// TEMPORARY copy of Insight's AxisViewportInt32 until Dev-Core copy up is complete and it becomes public

#pragma once

#include "Templates/SharedPointer.h"

class SScrollBar;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNPAxisViewportDouble
{
private:
	static constexpr float SLATE_UNITS_TOLERANCE = 0.1f;

public:
	FNPAxisViewportDouble()
	{
		Reset();
	}

	void Reset()
	{
		Size = 0.0f;

		MinValue = 0.0;
		MaxValue = 0.0;

		MinPosition = 0.0f;
		MaxPosition = 0.0f;
		Position = 0.0f;

		MinScale = 0.01;
		MaxScale = 100.0;
		Scale = 1.0;
	}

	FString ToDebugString(const TCHAR* Sufix) const
	{
		return FString::Printf(TEXT("Scale%s: %g, Pos%s: %.2f"), Sufix, Scale, Sufix, Position);
	}

	/**
	 * Size of viewport, in pixels (Slate units).
	 * The viewport's width if this is a horizontal axis or the viewport's height if this is a vertical axis.
	 */
	float GetSize() const { return Size; }
	bool SetSize(const float InSize);

	int32 GetMinValue() const { return MinValue; }
	int32 GetMaxValue() const { return MaxValue; }

	float GetMinPos() const { return MinPosition; }
	float GetMaxPos() const { return MaxPosition; }
	float GetPos() const { return Position; }

	double GetMinScale() const { return MinScale; }
	double GetMaxScale() const { return MaxScale; }

	/** Current scale factor between Value units and Slate units (pixels), in pixels per value. [px/value_unit] */
	double GetScale() const { return Scale; }

	void SetMinMaxValueInterval(const double InMinValue, const double InMaxValue)
	{
		MinValue = InMinValue;
		MaxValue = InMaxValue;
		UpdateMinMax();
	}

	double GetValueAtPos(const float Pos) const
	{
		return static_cast<double>(Pos) / Scale;
	}

	double GetValueAtOffset(const float Offset) const
	{
		return static_cast<double>(Position + Offset) / Scale;
	}

	float GetPosForValue(const double Value) const
	{
		return static_cast<float>(Value * Scale);
	}

	float GetOffsetForValue(const double Value) const
	{
		return static_cast<float>(Value * Scale) - Position;
	}

	float GetRoundedOffsetForValue(const double Value) const
	{
		return FMath::RoundToFloat(static_cast<float>(Value * Scale) - Position);
	}

	bool ScrollAtPos(const float Pos)
	{
		if (Position != Pos)
		{
			Position = Pos;
			OnPositionChanged();
			return true;
		}
		return false;
	}

	bool ScrollAtValue(const double Value)
	{
		return ScrollAtPos(GetPosForValue(Value));
	}

	bool CenterOnValue(const double Value)
	{
		return ScrollAtPos(static_cast<float>(Value * Scale) - Size / 2.0f);
	}

	bool CenterOnValueInterval(const double IntervalStartValue, const double IntervalEndValue)
	{
		const float IntervalSize = static_cast<float>(Scale * (IntervalEndValue- IntervalStartValue));
		if (IntervalSize > Size)
		{
			return ScrollAtValue(IntervalStartValue);
		}
		else
		{
			return ScrollAtPos(static_cast<float>(IntervalStartValue * Scale) - (Size - IntervalSize) / 2.0f);
		}
	}

	void SetScaleLimits(const double InMinScale, const double InMaxScale)
	{
		MinScale = InMinScale;
		MaxScale = InMaxScale;
	}

	bool SetScale(const double NewScale);
	bool ZoomWithFixedOffset(const double NewScale, const float Offset);
	bool RelativeZoomWithFixedOffset(const float Delta, const float Offset);

	void GetScrollLimits(float& OutMinPos, float& OutMaxPos);
	bool EnforceScrollLimits(const float InMinPos, const float InMaxPos, const float InterpolationFactor);
	bool UpdatePosWithinLimits();

	void OnUserScrolled(TSharedPtr<SScrollBar> ScrollBar, float ScrollOffset);
	void UpdateScrollBar(TSharedPtr<SScrollBar> ScrollBar) const;

private:
	void OnSizeChanged()
	{
	}

	void OnPositionChanged()
	{
	}

	void OnScaleChanged()
	{
		UpdateMinMax();
	}

	void UpdateMinMax()
	{
		MinPosition = GetPosForValue(MinValue);
		MaxPosition = GetPosForValue(MaxValue);

		//const float NewPosition = FMath::Clamp(Position, MinPosition, MaxPosition);
		//if (FMath::IsNearlyEqual(NewPosition, Position, SLATE_UNITS_TOLERANCE))
		//{
		//	Position = NewPosition;
		//	OnPositionChanged();
		//}
	}

private:
	float Size; // size of viewport, in pixels (Slate units); [px]

	double MinValue; // minimum value (inclusive); [value_unit]
	double MaxValue; // maximum value (exclusive); [value_unit]

	//double StartValue; // value of viewport's left side; [value_unit]; StartValue = GetValueAtPos(Position); computed when Position or Scale changes
	//double EndValue; // value of viewport's right side; [value_unit]; EndValue = GetValueAtPos(Position + Size); computed when Position, Size or Scale changes

	float MinPosition; // minimum position (corresponding to MinValue), in pixels (Slate units); [px]
	float MaxPosition; // maximum position (corresponding to MaxValue), in pixels (Slate units); [px]
	float Position; // current position of the viewport, in pixels (Slate units); [px]

	double MinScale; // minimum scale factor; [px/value_unit]
	double MaxScale; // maximum scale factor; [px/value_unit]
	double Scale; // current scale factor between Value units and Slate units (pixels); [px/value_unit]
};

////////////////////////////////////////////////////////////////////////////////////////////////////
