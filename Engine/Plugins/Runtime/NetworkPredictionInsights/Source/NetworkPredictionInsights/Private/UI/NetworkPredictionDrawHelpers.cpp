// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionDrawHelpers.h"


#include "InsightsCore/Common/PaintUtils.h"


void FNetworkPredictionDrawHelpers::DrawBackground(
	const FDrawContext& DrawContext,
	const FSlateBrush* BackgroundAreaBrush,
	const FLinearColor& ValidAreaColor,
	const FLinearColor& InvalidAreaColor,
	const FLinearColor& EdgeColor,
	const float X0,
	const float X1,
	const float X2,
	const float X3,
	const float XP,
	const float Y,
	const float H,
	float& OutValidAreaX,
	float& OutValidAreaW)
{
	//  <------- W ------->
	//  X0    X1 XP X2     X3
	//  ++++++|**1**|++++++
	//  ++++++|**1**|++++++
	//  ++++++|**1**|++++++

	const FLinearColor RightValidArea = ValidAreaColor * 1.1f;

	if (X1 >= X3 || X2 <= X0)
	{
		OutValidAreaX = X0;
		OutValidAreaW = X3 - X0;

		// Draw invalid area (entire view).
		DrawContext.DrawBox(X0, Y, X3 - X0, H, BackgroundAreaBrush, InvalidAreaColor);
	}
	else // X1 < X3 && X2 > X0
	{
		if (X1 > X0)
		{
			// Draw invalid area (left).
			DrawContext.DrawBox(X0, Y, X1 - X0, H, BackgroundAreaBrush, InvalidAreaColor);
		}

		if (X2 < X3)
		{
			// Draw invalid area (right).
			DrawContext.DrawBox(X2 + 1.0f, Y, X3 - X2 - 1.0f, H, BackgroundAreaBrush, InvalidAreaColor);

			// Draw the right edge (end time).
			DrawContext.DrawBox(X2, Y, 1.0f, H, BackgroundAreaBrush, EdgeColor);
		}

		float ValidAreaX = FMath::Max(X1, X0);
		float ValidAreaW = FMath::Min(X2, X3) - ValidAreaX;

		if (X1 >= X0)
		{
			// Draw the left edge (start time).
			DrawContext.DrawBox(X1, Y, 1.0f, H, BackgroundAreaBrush, EdgeColor);

			// Adjust valid area to not overlap the left edge.
			ValidAreaX += 1.0f;
			ValidAreaW -= 1.0f;
		}

		if (ValidAreaW > 0.0f)
		{
			// Draw valid area.
			const float LeftValidW = XP - ValidAreaX;
			const float RightValidW = ValidAreaW - LeftValidW;

			DrawContext.DrawBox(ValidAreaX, Y, LeftValidW, H, BackgroundAreaBrush, ValidAreaColor);

			DrawContext.DrawBox(XP, Y, 1.0f, H, BackgroundAreaBrush, EdgeColor);
			DrawContext.DrawBox(XP+1, Y, RightValidW, H, BackgroundAreaBrush, RightValidArea);
		}

		OutValidAreaX = ValidAreaX;
		OutValidAreaW = ValidAreaW;
	}

	DrawContext.LayerId++;
}


void FNetworkPredictionDrawHelpers::DrawBackground(const FDrawContext& DrawContext,
								  const FSlateBrush* BackgroundAreaBrush,
								  const float X0,
								  const float X1,
								  const float X2,
								  const float X3,
								  const	float X4,
								  const float Y,
								  const float H)
{
	const FLinearColor ValidAreaColor(0.07f, 0.07f, 0.07f, 1.0f);
	const FLinearColor InvalidAreaColor(0.1f, 0.07f, 0.07f, 1.0f);
	const FLinearColor EdgeColor(0.05f, 0.05f, 0.05f, 1.0f);

	float ValidAreaX, ValidAreaW;
	FNetworkPredictionDrawHelpers::DrawBackground(DrawContext, BackgroundAreaBrush, ValidAreaColor, InvalidAreaColor, EdgeColor, X0, X1, X2, X3, X4, Y, H, ValidAreaX, ValidAreaW);
}


void FNetworkPredictionDrawHelpers::DrawFutureBackground (	const FDrawContext& DrawContext,
										const FSlateBrush* BackgroundAreaBrush,
										const FLinearColor& Color,
										const float X0,
										const float X1,
										const float Y,
										const float H)
{
	DrawContext.DrawBox(X0, Y, X1 - X0, H, BackgroundAreaBrush, Color);
	DrawContext.LayerId++;
}
