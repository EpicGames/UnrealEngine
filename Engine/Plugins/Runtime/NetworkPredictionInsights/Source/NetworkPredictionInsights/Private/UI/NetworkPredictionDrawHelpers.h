// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


struct FLinearColor;

struct FDrawContext;
struct FSlateBrush;

class FNetworkPredictionDrawHelpers
{
public:
	/**
	 * Draw background.
	 * X1-X2 specifies the valid area. X0-X1 and X2-X3 specifies the invalid areas.
	 *
	 *  X0    X1    X2     X3
	 *  ++++++|*****|++++++
	 *  ++++++|*****|++++++
	 *  ++++++|*****|++++++
	 */
	static void DrawBackground(const FDrawContext& DrawContext,
							   const FSlateBrush* BackgroundAreaBrush,
							   const FLinearColor& ValidAreaColor,
							   const FLinearColor& InvalidAreaColor,
							   const FLinearColor& EdgeColor,
							   const float X0,
							   const float X1,
							   const float X2,
							   const float X3,
							   const float X4,
							   const float Y,
							   const float H,
							   float& OutValidAreaX,
							   float& OutValidAreaW);

	static void DrawBackground(	const FDrawContext& DrawContext,
								const FSlateBrush* BackgroundAreaBrush,
								const float X0,
								const float X1,
								const float X2,
								const float X3,
								const float X4,
								const float Y,
								const float H);

	static void DrawFutureBackground (	const FDrawContext& DrawContext,
										const FSlateBrush* BackgroundAreaBrush,
										const FLinearColor& Color,
										const float X0,
										const float X1,
										const float Y,
										const float H);
};
