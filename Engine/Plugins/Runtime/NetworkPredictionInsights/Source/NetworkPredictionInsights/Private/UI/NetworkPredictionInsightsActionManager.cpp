// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionInsightsActionManager.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

#define IMPLEMENT_TOGGLE_COMMAND(CmdName, IsEnabled, SetIsEnabled) \
	\
	void FNetworkPredictionInsightsActionManager::Map_##CmdName##_Global()\
	{\
		This->CommandList->MapAction(This->GetCommands().CmdName, CmdName##_Custom());\
	}\
	\
	const FUIAction FNetworkPredictionInsightsActionManager::CmdName##_Custom() \
	{\
		FUIAction UIAction;\
		UIAction.ExecuteAction = FExecuteAction::CreateRaw(this, &FNetworkPredictionInsightsActionManager::CmdName##_Execute);\
		UIAction.CanExecuteAction = FCanExecuteAction::CreateRaw(this, &FNetworkPredictionInsightsActionManager::CmdName##_CanExecute);\
		UIAction.GetActionCheckState = FGetActionCheckState::CreateRaw(this, &FNetworkPredictionInsightsActionManager::CmdName##_GetCheckState);\
		return UIAction;\
	}\
	\
	void FNetworkPredictionInsightsActionManager::CmdName##_Execute()\
	{\
		const bool b##IsEnabled = !This->IsEnabled();\
		This->SetIsEnabled(b##IsEnabled);\
	}\
	\
	bool FNetworkPredictionInsightsActionManager::CmdName##_CanExecute() const\
	{\
		return FInsightsManager::Get()->GetSession().IsValid();\
	}\
	\
	ECheckBoxState FNetworkPredictionInsightsActionManager::CmdName##_GetCheckState() const\
	{\
		const bool b##IsEnabled = This->IsEnabled();\
		return b##IsEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;\
	}

//IMPLEMENT_TOGGLE_COMMAND(ToggleAutoScrollSimulationFrames, AutoScrollSimulationFrames, SetAutoScrollSimulationFrames)

#undef IMPLEMENT_TOGGLE_COMMAND
