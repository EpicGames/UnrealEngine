// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/**
 * Class that provides helper functions for the commands to avoid cluttering profiler manager with many small functions.
 * Can't contain any variables. Directly operates on the profiler manager instance.
 */
class FNetworkPredictionInsightsActionManager
{
	friend class FNetworkPredictionInsightsManager;

private:
	/** Private constructor. */
	FNetworkPredictionInsightsActionManager(class FNetworkPredictionInsightsManager* Instance)
		: This(Instance)
	{}

	//////////////////////////////////////////////////
	// Toggle Commands

#define DECLARE_TOGGLE_COMMAND(CmdName)\
public:\
	void Map_##CmdName##_Global(); /**< Maps UI command info CmdName with the specified UI command list. */\
	const FUIAction CmdName##_Custom(); /**< UI action for CmdName command. */\
protected:\
	void CmdName##_Execute(); /**< Handles FExecuteAction for CmdName. */\
	bool CmdName##_CanExecute() const; /**< Handles FCanExecuteAction for CmdName. */\
	ECheckBoxState CmdName##_GetCheckState() const; /**< Handles FGetActionCheckState for CmdName. */

	//DECLARE_TOGGLE_COMMAND(ToggleAutoScrollSimulationFrames)
#undef DECLARE_TOGGLE_COMMAND



	//////////////////////////////////////////////////

protected:
	/** Reference to the global instance of the profiler manager. */
	class FNetworkPredictionInsightsManager* This;
};
