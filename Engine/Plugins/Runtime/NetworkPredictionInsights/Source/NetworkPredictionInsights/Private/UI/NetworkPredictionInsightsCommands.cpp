// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionInsightsCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "NetworkPredictionCommands"

FNetworkPredictionInsightsCommands::FNetworkPredictionInsightsCommands()
	: TCommands<FNetworkPredictionInsightsCommands>(
		TEXT("NetworkPredictionCommand"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "NetworkPredictionCommand", "Network Prediction Insights"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}

// UI_COMMAND takes long for the compiler to optimize
UE_DISABLE_OPTIMIZATION_SHIP

void FNetworkPredictionInsightsCommands::RegisterCommands()
{
	UI_COMMAND(ToggleAutoScrollSimulationFrames, "AutoScroll", "Toggles auto scrolling in the simulation frame view when new data is received.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::SpaceBar));
	UI_COMMAND(NextEngineFrame, ">", "Advances Simulation view by one Engine Frame", EUserInterfaceActionType::Button, FInputChord(EKeys::Period));
	UI_COMMAND(PrevEngineFrame, "<", "Rewinds the Simulation view by one Engine Frame", EUserInterfaceActionType::Button, FInputChord(EKeys::Comma));
	UI_COMMAND(FirstEngineFrame, "<<", "Jump to the first frame in the Simulation View", EUserInterfaceActionType::Button, FInputChord(EKeys::Home));
	UI_COMMAND(LastEngineFrame, ">>", "Jump to the last frame in the Simulation View", EUserInterfaceActionType::Button, FInputChord(EKeys::End));
}
UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE