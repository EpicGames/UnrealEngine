// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FNetworkPredictionInsightsCommands : public TCommands<FNetworkPredictionInsightsCommands>
{
public:
	/** Default constructor. */
	FNetworkPredictionInsightsCommands();

	/** Initialize commands. */
	virtual void RegisterCommands() override;

public:
	//////////////////////////////////////////////////
	// Global commands need to implement following method:
	//     void Map_<CommandName>_Global();
	// Custom commands needs to implement also the following method:
	//     const FUIAction <CommandName>_Custom(...) const;
	//////////////////////////////////////////////////
	
	TSharedPtr<FUICommandInfo> ToggleAutoScrollSimulationFrames;
	TSharedPtr<FUICommandInfo> ToggleCompactView;


	TSharedPtr<FUICommandInfo> NextEngineFrame;
	TSharedPtr<FUICommandInfo> PrevEngineFrame;
	TSharedPtr<FUICommandInfo> FirstEngineFrame;
	TSharedPtr<FUICommandInfo> LastEngineFrame;
};

