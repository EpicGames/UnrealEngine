// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionInsightsManager.h"
#include "SNPWindow.h"
#include "NetworkPredictionInsightsCommands.h"

TSharedPtr<FNetworkPredictionInsightsManager> FNetworkPredictionInsightsManager::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

FNetworkPredictionInsightsManager::FNetworkPredictionInsightsManager()
	: ActionManager(this)
{
}

void FNetworkPredictionInsightsManager::PostConstructor()
{
	FNetworkPredictionInsightsCommands::Register();
	BindCommands();
}

void FNetworkPredictionInsightsManager::BindCommands()
{
}

FNetworkPredictionInsightsManager::~FNetworkPredictionInsightsManager()
{
	FNetworkPredictionInsightsCommands::Unregister();
}

TSharedPtr<FNetworkPredictionInsightsManager> FNetworkPredictionInsightsManager::Get()
{
	return FNetworkPredictionInsightsManager::Instance;
}

const FNetworkPredictionInsightsCommands& FNetworkPredictionInsightsManager::GetCommands()
{
	return FNetworkPredictionInsightsCommands::Get();
}

FNetworkPredictionInsightsActionManager& FNetworkPredictionInsightsManager::GetActionManager()
{
	return FNetworkPredictionInsightsManager::Instance->ActionManager;
}

bool FNetworkPredictionInsightsManager::Tick(float DeltaTime)
{
	return true;
}

void FNetworkPredictionInsightsManager::AddProfilerWindow(const TSharedRef<SNPWindow>& InWindow)
{
	NetworkPredictionInsightsWindows.Add(InWindow);
}

void FNetworkPredictionInsightsManager::RemoveProfilerWindow(const TSharedRef<SNPWindow>& InWindow)
{
	NetworkPredictionInsightsWindows.Remove(InWindow);
}
	
TSharedPtr<class SNPWindow> FNetworkPredictionInsightsManager::GetProfilerWindow(int32 Index) const
{
	return NetworkPredictionInsightsWindows[Index].Pin();
}

void FNetworkPredictionInsightsManager::OnSessionChanged()
{
	for (TWeakPtr<SNPWindow> WndWeakPtr : NetworkPredictionInsightsWindows)
	{
		TSharedPtr<SNPWindow> Wnd = WndWeakPtr.Pin();
		if (Wnd.IsValid())
		{
			Wnd->Reset();
		}
	}
}