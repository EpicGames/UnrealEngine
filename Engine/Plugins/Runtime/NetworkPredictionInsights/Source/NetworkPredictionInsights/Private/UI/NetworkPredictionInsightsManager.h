// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionInsightsActionManager.h"
#include "Containers/Ticker.h"

class SNPWindow;

/**
 * This class manages the Networking Profiler state and settings.
 */
class FNetworkPredictionInsightsManager : public TSharedFromThis<FNetworkPredictionInsightsManager>
{
	friend class FNetworkPredictionInsightsActionManager;

public:
	/** Creates the Networking Profiler (Networking Insights) manager, only one instance can exist. */
	FNetworkPredictionInsightsManager();

	/** Virtual destructor. */
	virtual ~FNetworkPredictionInsightsManager();

	/** Creates an instance of the profiler manager. */
	static TSharedPtr<FNetworkPredictionInsightsManager> Initialize()
	{
		if (FNetworkPredictionInsightsManager::Instance.IsValid())
		{
			FNetworkPredictionInsightsManager::Instance.Reset();
		}

		FNetworkPredictionInsightsManager::Instance = MakeShared<FNetworkPredictionInsightsManager>();
		FNetworkPredictionInsightsManager::Instance->PostConstructor();

		return FNetworkPredictionInsightsManager::Instance;
	}

	/** Shutdowns the Networking Profiler manager. */
	void Shutdown()
	{
		FNetworkPredictionInsightsManager::Instance.Reset();
	}

protected:
	/** Finishes initialization of the profiler manager. */
	void PostConstructor();

	/** Binds our UI commands to delegates. */
	void BindCommands();

public:
	/**
	 * @return the global instance of the Network Prediction Insights manager.
	 * This is an internal singleton and cannot be used outside NetworkPredictionInsights module.
	 */
	static TSharedPtr<FNetworkPredictionInsightsManager> Get();

	/** @returns UI command list for the Networking Profiler manager. */
	//const TSharedRef<FUICommandList> GetCommandList() const;

	/** @return an instance of the Networking Profiler commands. */
	static const class FNetworkPredictionInsightsCommands& GetCommands();

	/** @return an instance of the Networking Profiler action manager. */
	static FNetworkPredictionInsightsActionManager& GetActionManager();

	void AddProfilerWindow(const TSharedRef<SNPWindow>& InWindow);

	void RemoveProfilerWindow(const TSharedRef<SNPWindow>& InWindow);

	/**
	 * Converts profiler window weak pointer to a shared pointer and returns it.
	 * Make sure the returned pointer is valid before trying to dereference it.
	 */
	TSharedPtr<class SNPWindow> GetProfilerWindow(int32 Index) const;
	
	////////////////////////////////////////////////////////////////////////////////////////////////////

	void OnSessionChanged();

protected:
	/** Updates this manager, done through FCoreTicker. */
	bool Tick(float DeltaTime);

protected:
	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FDelegateHandle OnTickHandle;

	/** List of UI commands for this manager. This will be filled by this and corresponding classes. */
	//TSharedRef<FUICommandList> CommandList;

	/** An instance of the Networking Profiler action manager. */
	FNetworkPredictionInsightsActionManager ActionManager;

	/** A list of weak pointers to the Networking Profiler windows. */
	TArray<TWeakPtr<class SNPWindow>> NetworkPredictionInsightsWindows;

	/** A shared pointer to the global instance of the profiler manager. */
	static TSharedPtr<FNetworkPredictionInsightsManager> Instance;
};
