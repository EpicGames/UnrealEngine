// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNPSimFrameContents.h"
#include "UI/SNPWindow.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Framework/Text/RichTextLayoutMarshaller.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SNPSimFrameContents"

SNPSimFrameContents::SNPSimFrameContents()
{

}
SNPSimFrameContents::~SNPSimFrameContents()
{

}

void SNPSimFrameContents::Reset()
{

}

void SNPSimFrameContents::Construct(const FArguments& InArgs, TSharedPtr<SNPWindow> InNPWindow)
{
	NPWindow = InNPWindow;
	ChildSlot
	[
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding(FMargin(0, 0, 0, 0))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(2.0f)
			.AutoHeight()
			[
				SAssignNew(ContentsHBoxPtr, SHorizontalBox)
			]

			+SVerticalBox::Slot()
			.Padding(2.0f)
			.AutoHeight()
			[
				SAssignNew(SystemFaultsVBoxPtr, SVerticalBox)
			]
		]
	];
}

void SNPSimFrameContents::NotifyContentClicked(const FSimContentsView& InContent)
{
	struct FUserStateWidgetInfo
	{
		const FSimulationData::FUserState* State = nullptr;
		const TCHAR* Heading = nullptr;
		FLinearColor Color = FLinearColor::White;
	};

	auto MakeUserStateWidget = [&](const FUserStateWidgetInfo& Info, const TCHAR* HighlightUserStr, bool bHighlightAsWarning)
	{
		FString HeaderText;
		FString UserStr;
		uint64 EngineFrameHyperLink = 0;
		if (Info.State)
		{
			HeaderText = FString::Printf(TEXT("%s [Sim: %d. <a id=\"engine\">Engine: %" UINT64_FMT "</>. %s]"), Info.Heading, Info.State->SimFrame, Info.State->EngineFrame, LexToString(Info.State->Source));
			UserStr = Info.State->UserStr;
			
			if (HighlightUserStr)
			{
				const int32 StartIdx = UserStr.Find(HighlightUserStr);
				if (StartIdx != INDEX_NONE)
				{
					if (bHighlightAsWarning)
					{
						UserStr.InsertAt(StartIdx, TEXT("<TextBlock.ShadowedTextWarning>"));
					}
					else
					{
						UserStr.InsertAt(StartIdx, TEXT("<RichTextBlock.Bold>"));
					}
					
					const int32 EndIdx = UserStr.Find(TEXT("\n"), ESearchCase::IgnoreCase, ESearchDir::FromStart, StartIdx);
					UserStr.InsertAt(EndIdx == INDEX_NONE ? UserStr.Len()-1 : EndIdx, TEXT("</>"));
				}
			}

			if (Info.State->OOBStr)
			{
				UserStr += FString::Printf(TEXT("\nOOB Mod:\n%s"), Info.State->OOBStr);
			}

			EngineFrameHyperLink = Info.State->EngineFrame;
		}
		else
		{
			HeaderText = FString::Printf(TEXT("%s [State not found!]"), Info.Heading);
		}
		
		const FTextBlockStyle& TextStyle = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

		TSharedRef<FRichTextLayoutMarshaller> RichTextMarshaller = FRichTextLayoutMarshaller::Create(
			TArray<TSharedRef<ITextDecorator>>(), 
			//&FCoreStyle::Get()
			&FAppStyle::Get()
			);		

		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.Padding(0.0f, 4.0f, 0.0f, 4.0f)
			[
				/*
				SNew(STextBlock)
				.Text(FText::FromString(HeaderText))
				.ColorAndOpacity(Info.Color)
				*/
				SNew( SRichTextBlock )
				.Text(FText::FromString(HeaderText))
				//.HighlightText( FText::FromString(TEXT("Loc:")))
				//.TextStyle(&TextStyle)
				//.AutoWrapText(true)
				//.DecoratorStyleSet( &FAppStyle::Get() )
				.DecoratorStyleSet( &FCoreStyle::Get() )
				+SRichTextBlock::HyperlinkDecorator( TEXT("engine"), FSlateHyperlinkRun::FOnClick::CreateLambda([this, EngineFrameHyperLink](const FSlateHyperlinkRun::FMetadata& Metadata) { NPWindow->SetEngineFrame(EngineFrameHyperLink, true); } ) )
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				/*
				SNew(SEditableText)
				.IsReadOnly(true)
				.Text(FText::FromString(UserStr))
				.ColorAndOpacity(Info.Color)
				*/

				SNew(SMultiLineEditableText)
				.Text(FText::FromString(UserStr))
				.Marshaller(RichTextMarshaller)
				.IsReadOnly(true)
			];
			
	};

	auto AddUserStateVSlots = [&](const TArrayView<const FUserStateWidgetInfo>& Source, const TCHAR* HighlightUserStr=nullptr, bool bHighlightAsWarning=false)
	{
		TSharedPtr<SVerticalBox> VBox;
		ContentsHBoxPtr->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		[
			SAssignNew(VBox, SVerticalBox)
		];

		auto ConditionalMake = [&](const FUserStateWidgetInfo& Info)
		{
			if (Info.State)
			{
				VBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.Padding(4.0f, 4.0f, 4.0f, 4.0f)
				[
					MakeUserStateWidget(Info, HighlightUserStr, bHighlightAsWarning)
				];
			}
		};

		for (const FUserStateWidgetInfo& Info : Source)
		{
			ConditionalMake(Info);
		}
	};

	
	ContentsHBoxPtr->ClearChildren();
	
	// -------------------------------------------------------------
	// Simulation Info
	// -------------------------------------------------------------
	FString GroupNameStr = InContent.SimView->ConstData.GroupName.ToString();
	FString NetRoleStr = LexToString(InContent.SimView->SparseData->NetRole);

	FString SimInfoText;
	SimInfoText += FString::Format(TEXT(
		"{0}\n"
		"Sim Group: {1}\n"
		"SimID: {2}\n"
		"TraceID: {3}\n"
		"NetRole: {4}\n"
		"ServiceMask: {5}\n"
		"TickingMode: {6}\n"
		""),
	{
		*InContent.SimView->ConstData.DebugName,
		*GroupNameStr,
		InContent.SimView->ConstData.ID.SimID,
		InContent.SimView->TraceID,
		*NetRoleStr,
		InContent.SimView->SparseData->ServiceMask,
		LexToString(InContent.SimView->SparseData->TickingPolicy)
	});

	TSharedPtr<SVerticalBox> VertBox;

	ContentsHBoxPtr->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Top)
	.HAlign(HAlign_Left)
	.Padding(4.0f, 4.0f, 4.0f, 4.0f)
	[
		SAssignNew(VertBox, SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding(4.0f, 4.0f, 4.0f, 4.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(SimInfoText))
		]
	];

	// -------------------------------------------------------------
	//	Simulation Tick
	// -------------------------------------------------------------
	if (InContent.SimTick)
	{
		TSharedRef<FRichTextLayoutMarshaller> RichTextMarshaller = FRichTextLayoutMarshaller::Create(
			TArray<TSharedRef<ITextDecorator>>(), &FAppStyle::Get());

		FString TickText;

		const FSimulationData::FTick& SimTick = *InContent.SimTick;
		FString ReconcileStr;

		bool bHighlightAsWarning = true;

		if (SimTick.bReconcileDueToOffsetChange || SimTick.ReconcileStr)
		{
			ReconcileStr = "Required Reconcile:\n";
			if (SimTick.bReconcileDueToOffsetChange)
			{
				bHighlightAsWarning = false; // Offset change is what caused offset, not user code 
				ReconcileStr += TEXT("  <TextBlock.ShadowedTextWarning>InputCmd Offset Changed!</>\n");
			}
			if (SimTick.ReconcileStr)
			{
				ReconcileStr += TEXT("  ");
				ReconcileStr += SimTick.ReconcileStr;
			}
		}

		TickText += FString::Format(TEXT(
			"Simulation Frame {0}\n"
			"Local Offset {1}\n"
			"Start Simulation MS: {2}\n"
			"End Simulation MS: {3}\n"
			"Delta Simulation MS: {4}\n"
			"Num Buffered InputCmds: {5}\n"
			"Input Buffer Fault: {6}\n"
			"Repredict: {7}\n"
			"Confirmed Engine Frame: {8}\n"
			"Trashed Engine Frame: {9}\n"
			"GFrameNumber: {10}\n\n"
			"{11}\n\n"),
		{
			SimTick.OutputFrame,
			SimTick.LocalOffsetFrame,
			SimTick.StartMS,
			SimTick.EndMS,
			(SimTick.EndMS - SimTick.StartMS),
			SimTick.NumBufferedInputCmds,
			SimTick.bInputFault,
			SimTick.bRepredict,
			SimTick.ConfirmedEngineFrame,
			SimTick.TrashedEngineFrame,
			SimTick.EngineFrame,
			ReconcileStr
		});

		VertBox->AddSlot()

		.AutoHeight()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(4.0f, 4.0f, 4.0f, 4.0f)
		[
			/*
			SNew(STextBlock)
			.Text(FText::FromString(TickText))
			*/
			SNew(SMultiLineEditableText)
			.Text(FText::FromString(TickText))
			.Marshaller(RichTextMarshaller)
			.IsReadOnly(true)
		];


		// Input/Output states
		
		const int32 InputFrame = SimTick.OutputFrame - 1;

		const FSimulationData::FUserState* InputCmd = InContent.SimView->UserData.Get(ENP_UserState::Input, InputFrame, InContent.SimTick->EngineFrame, (uint8)ENP_UserStateSource::NetRecv);
		
		const FSimulationData::FUserState* InSyncState = InContent.SimView->UserData.Get(ENP_UserState::Sync, InputFrame, InContent.SimTick->EngineFrame, (uint8)ENP_UserStateSource::NetRecv);
		const FSimulationData::FUserState* InAuxState = InContent.SimView->UserData.Get(ENP_UserState::Aux, InputFrame, InContent.SimTick->EngineFrame, (uint8)ENP_UserStateSource::NetRecv);

		const FSimulationData::FUserState* OutSyncState = InContent.SimView->UserData.Get(ENP_UserState::Sync, SimTick.OutputFrame, InContent.SimTick->EngineFrame, (uint8)ENP_UserStateSource::NetRecv);
		const FSimulationData::FUserState* OutAuxState = InContent.SimView->UserData.Get(ENP_UserState::Aux, SimTick.OutputFrame, InContent.SimTick->EngineFrame, (uint8)ENP_UserStateSource::NetRecv);

		const FSimulationData::FUserState* InPhysicsState = InContent.SimView->UserData.Get(ENP_UserState::Physics, InputFrame, InContent.SimTick->EngineFrame, (uint8)ENP_UserStateSource::NetRecv);

		AddUserStateVSlots( { FUserStateWidgetInfo{ InputCmd, TEXT("Input Cmd"), FLinearColor::White } });
		AddUserStateVSlots( { FUserStateWidgetInfo{ InSyncState, TEXT("In Sync"), FLinearColor::White }, FUserStateWidgetInfo{ InAuxState, TEXT("In Aux"), FLinearColor::White } }, SimTick.ReconcileStr, bHighlightAsWarning);

		AddUserStateVSlots( { FUserStateWidgetInfo{ InPhysicsState, TEXT("Physics"), FLinearColor::White } }, SimTick.ReconcileStr, bHighlightAsWarning);

		if (InAuxState != OutAuxState)
		{
			AddUserStateVSlots( { FUserStateWidgetInfo{ OutSyncState, TEXT("Out Sync"), FLinearColor::White }, FUserStateWidgetInfo{ OutAuxState, TEXT("Out Aux"), FLinearColor::White } });
		}
		else
		{
			AddUserStateVSlots( { FUserStateWidgetInfo{ OutSyncState, TEXT("Out Sync"), FLinearColor::White }, FUserStateWidgetInfo{ OutAuxState, TEXT("Out Aux (No Change)"), FLinearColor::Gray} });
		}
	}

	// -------------------------------------------------------------
	//	Net Receive
	// -------------------------------------------------------------

	if (InContent.NetRecv)
	{
		const FSimulationData::FNetSerializeRecv& NetRecv = *InContent.NetRecv;

		FString NetRecvText = FString::Format(TEXT(
			"Net Receive:\n"
			"SimFrame: {0}\n"
			"SimTime MS: {1}\n"
			"Status:  {2}\n"
			"GFrameNumber: {3}\n"),
		{
			NetRecv.Frame,
			NetRecv.SimTimeMS,
			LexToString(NetRecv.Status),
			NetRecv.EngineFrame
		});		

		ContentsHBoxPtr->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(4.0f, 0.0f, 4.0f, 0.0f)
		[
			SAssignNew(VertBox, SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.Padding(4.0f, 4.0f, 4.0f, 4.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(NetRecvText))
			]
		];

		uint8 Mask = ~((uint8)ENP_UserStateSource::NetRecv | (uint8)ENP_UserStateSource::NetRecvCommit);
		const FSimulationData::FUserState* RecvInputCmd = InContent.SimView->UserData.Get(ENP_UserState::Input, InContent.NetRecv->Frame, InContent.NetRecv->EngineFrame, Mask);
		const FSimulationData::FUserState* RecvSyncState = InContent.SimView->UserData.Get(ENP_UserState::Sync, InContent.NetRecv->Frame, InContent.NetRecv->EngineFrame, Mask);
		const FSimulationData::FUserState* RecvAuxState = InContent.SimView->UserData.Get(ENP_UserState::Aux, InContent.NetRecv->Frame, InContent.NetRecv->EngineFrame, Mask);
		const FSimulationData::FUserState* RecvPhysicsState = InContent.SimView->UserData.Get(ENP_UserState::Physics, InContent.NetRecv->Frame, InContent.NetRecv->EngineFrame, Mask);

		auto SelectColor = [](const FSimulationData::FUserState* UserState)
		{
			return UserState && UserState->Source == ENP_UserStateSource::NetRecvCommit ? FLinearColor::White : FLinearColor::Gray;
		};

		AddUserStateVSlots( { 
			FUserStateWidgetInfo{ RecvInputCmd, TEXT("Recv InputCmd"), SelectColor(RecvInputCmd) },
			FUserStateWidgetInfo{ RecvSyncState, TEXT("Recv Sync"), SelectColor(RecvSyncState) }, 
			FUserStateWidgetInfo{ RecvAuxState, TEXT("Recv Aux"), SelectColor(RecvAuxState) },
			FUserStateWidgetInfo{ RecvPhysicsState, TEXT("Recv Physics"), SelectColor(RecvAuxState) },
		});
	}


	// -------------------------------------
	// System Faults
	// -------------------------------------

	SystemFaultsVBoxPtr->ClearChildren();

	if (InContent.NetRecv && InContent.NetRecv->SystemFaults.Num() > 0)
	{
		FString FaultString = TEXT("NetRecv Faults:\n");

		for (const FSimulationData::FSystemFault& Fault : InContent.NetRecv->SystemFaults)
		{
			FaultString.Append(Fault.Str);
			FaultString.Append(TEXT("\n"));
		}

		SystemFaultsVBoxPtr->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(4.0f, 4.0f, 4.0f, 4.0f)
		[
			SNew(SEditableText)
			.IsReadOnly(true)
			.Text(FText::FromString(FaultString))
		];
	}
}



#undef LOCTEXT_NAMESPACE
