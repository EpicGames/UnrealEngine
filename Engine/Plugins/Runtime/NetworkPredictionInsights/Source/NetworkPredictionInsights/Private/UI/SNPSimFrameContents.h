// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SHorizontalBox;
class STextBlock;
class SVerticalBox;
struct FSimContentsView;

class SNPWindow;

class SNPSimFrameContents : public SCompoundWidget
{
public:

	SNPSimFrameContents();
	virtual ~SNPSimFrameContents();

	void Reset();

	SLATE_BEGIN_ARGS(SNPSimFrameContents) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SNPWindow> InNPWindow);

	void NotifyContentClicked(const FSimContentsView& InView);

	TSharedPtr<SHorizontalBox> ContentsHBoxPtr;
	TSharedPtr<SVerticalBox> SystemFaultsVBoxPtr;

	TSharedPtr<STextBlock> SimInfoTextBlock;
	TSharedPtr<STextBlock> SimTickTextBlock;

	TSharedPtr<STextBlock> InputCmdTextBlock;
	TSharedPtr<STextBlock> SyncStateTextBlock;
	TSharedPtr<STextBlock> AuxStateTextBlock;

	TSharedPtr<SNPWindow> NPWindow;
};
