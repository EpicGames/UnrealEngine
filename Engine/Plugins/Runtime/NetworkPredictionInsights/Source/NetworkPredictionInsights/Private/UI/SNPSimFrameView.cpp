// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNPSimFrameView.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "Modules/ModuleManager.h"
#include "NetworkPredictionDrawHelpers.h"
#include "SNPWindow.h"
#include "UI/NPAxisViewportDouble.h"
#include "UI/NPAxisViewportInt32.h"
#include "Widgets/Layout/SScrollBar.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/IUnrealInsightsModule.h"

#define LOCTEXT_NAMESPACE "NetworkPredictionInsights"

FSimFrameViewDrawHelper::FSimFrameViewDrawHelper(const FDrawContext& InDrawContext, const FSimFrameViewport& InViewport)
	: DrawContext(InDrawContext)
	, Viewport(InViewport)
	, WhiteBrush(FAppStyle::Get().GetBrush("WhiteBrush"))
	, HoveredEventBorderBrush(FAppStyle::Get().GetBrush("HoveredEventBorder"))
	, SelectedEventBorderBrush(FAppStyle::Get().GetBrush("SelectedEventBorder"))
	, SelectionFont(FCoreStyle::GetDefaultFontStyle("Regular", 8))
{

}


void FSimFrameViewDrawHelper::DrawBackground(FSimTime Presentable) const
{
	const FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	const float X0 = 0.0f;
	const float X1 = ViewportX.GetMinPos() - ViewportX.GetPos();
	const float X2 = ViewportX.GetMaxPos() - ViewportX.GetPos();
	const float X3 = FMath::CeilToFloat(Viewport.GetWidth());

	const float Y = 0.0f;
	const float H = FMath::CeilToFloat(Viewport.GetHeight());

	const float X4 = ViewportX.GetOffsetForValue(Presentable);

	FNetworkPredictionDrawHelpers::DrawBackground(DrawContext, WhiteBrush, X0, X1, X2, X3, X4, Y, H);
}

FLinearColor FSimFrameViewDrawHelper::GetFramColorByStatus(ESimFrameStatus Status, const bool bDesaturate) const
{
	auto GetBaseColor = [Status]() -> FLinearColor
	{
		constexpr float Alpha = 1.0f;
		switch (Status)
		{
		case ESimFrameStatus::Predicted:
			return FLinearColor(0.33f, 0.33f, 1.00f, Alpha);
		case ESimFrameStatus::Repredicted:		
			return FLinearColor(0.29f, 0.16f, 0.70f, Alpha);
		case ESimFrameStatus::Confirmed:
			return FLinearColor(0.33f, 1.0f, 0.33f, Alpha);
		case ESimFrameStatus::Trashed:
			return FLinearColor(1.0f, 0.33f, 0.33f, Alpha);		
		case ESimFrameStatus::Abandoned:
			return FLinearColor(0.25f, 0.25f, 0.25f, Alpha);
		default:
			return FLinearColor(1.0f, 1.0f, 1.0f, Alpha);
		}
	};

	FLinearColor Color = GetBaseColor();
	if (bDesaturate)
	{
		static float d = 0.2f;
		Color = Color.Desaturate(d);
	}
	return Color;
}

FLinearColor FSimFrameViewDrawHelper::GetRecvColorByStatus(ENetSerializeRecvStatus Status) const
{
	constexpr float Alpha = 1.0f;
	switch (Status)
	{
	case ENetSerializeRecvStatus::Unknown:
		return FLinearColor(1.0f, 1.0f, 1.0f, Alpha);

	case ENetSerializeRecvStatus::Confirm:
		return FLinearColor(0.25f, 1.f, 0.25f, Alpha);

	case ENetSerializeRecvStatus::Rollback:
		return FLinearColor(1.25f, 0.25f, 0.25f, Alpha);

	case ENetSerializeRecvStatus::Jump:
		return FLinearColor(1.f, 1.f, 0.25f, Alpha);

	case ENetSerializeRecvStatus::Fault:
		return FLinearColor(1.f, 0.25f, 1.f, Alpha);

	case ENetSerializeRecvStatus::Stale:
		return FLinearColor(0.25f, 0.25f, 0.25f, Alpha);

	default:
		return FLinearColor(1.0f, 0.0f, 0.0f, Alpha);
	}
}

void FSimFrameViewDrawHelper::DrawCached(const FSimulationFrameView& View, float MinY) const
{
	const FSlateFontInfo& Font = FCoreStyle::GetDefaultFontStyle("Regular", 8);
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const float MinWidthForEmbeddedText = FontMeasureService->Measure(TEXT("XXXXX"), Font).X; // Not great, would need to know highest output frame in all tracks which we currently don't cache anywhere

	const FLinearColor TextColor(0.69804f, 0.69804f, 0.69804f);
	float PulseFactor = FMath::Lerp<float>(0.25f, 1.0f, SNPSimFrameView::PulsePCT);
	const FLinearColor PulseColor = FLinearColor(PulseFactor, PulseFactor, PulseFactor, 1.f);
	const FLinearColor SelectedColor(1.0f, 0.5f, 0.0f);
	const FLinearColor SelectedPulseColor(1.0f * PulseFactor, 0.5f * PulseFactor, 0.0f);

	const FLinearColor SearchHighlightColor(1.0f, 1.0f, 0.15f);
	const FLinearColor SearchHighlightPulseColor(1.0f * PulseFactor, 1.0f * PulseFactor, 0.25f);

	auto SelectBorderColor = [&](const bool bPulse, const bool bSelected, const bool bSearchHighlighted, const FLinearColor& ColorFill)
	{
		if (bSelected)
		{
			return bPulse ? SelectedPulseColor : SelectedColor;
		}
		if (bSearchHighlighted)
		{
			return bPulse ? SearchHighlightPulseColor : SearchHighlightColor;
		}
		
		return bPulse ? PulseColor: FLinearColor(ColorFill.R * 0.25f, ColorFill.G * 0.25f, ColorFill.B * 0.25f, ColorFill.A);
	};

	auto DrawTick = [&](const FSimulationTrack::FSubTrack::FDrawTick& Tick, const float Y, const float H)
	{
		const FSimulationTrack::FSubTrack::FTickSource& Source = Tick.Source;

		const FLinearColor ColorFill = GetFramColorByStatus(Source.Status, Source.bDesaturate);
		const FLinearColor ColorBorder = SelectBorderColor(Source.bPulse, Source.bSelected, Source.bSearchHighlighted, ColorFill);
		const float X = Tick.X;
		const float W = Tick.W;

		if ( W > 2.0f)
		{
			DrawContext.DrawBox(X + 1.0f, Y + 1.0f, W - 2.0f, H - 2.0f, WhiteBrush, ColorFill);

			// Draw border.
			const float B = Source.bSearchHighlighted ? 6.f : ((Source.bPulse || Source.bSelected) ? 4.f : 2.f);
			const float B2 = B/2.f;
						
			DrawContext.DrawBox(X, Y, B2, H, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + W - B2, Y, B2, H, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + B2, Y, W - B, B2, WhiteBrush, ColorBorder);
			DrawContext.DrawBox(X + B2, Y + H - B2, W - B, B2, WhiteBrush, ColorBorder);
		}
		else
		{
			DrawContext.DrawBox(X, Y, W, H, WhiteBrush, ColorBorder);
		}

		if (W >= MinWidthForEmbeddedText)
		{
			const FString Text = View.ContentType == ESimFrameContentType::FrameNumber ? FString::Printf(TEXT("%d"), Source.Tick.OutputFrame) : FString::Printf(TEXT("%d"), Source.Tick.NumBufferedInputCmds);
			DrawContext.DrawText(X+2, Y, Text, Font, FLinearColor::Black);
		}
	};

	auto DrawNetRecv = [&](const FSimulationTrack::FSubTrack::FDrawNetRecv& NetRecv, const float Y, const float H)
	{
		const FSimulationTrack::FSubTrack::FNetRecvSource& Source = NetRecv.Source;

		const FLinearColor NetRecvColor = GetRecvColorByStatus(Source.NetRecv.Status);
		const FLinearColor NetRecvBorder(NetRecvColor.R * 0.75f, NetRecvColor.G * 0.75f, NetRecvColor.B * 0.75f, 1.f);

		const float X = NetRecv.X-1.f;
		const float W = NetRecv.W();
		const float NRY = Y-4.f;
		const float NRH = H+8.f;

		const bool DrawBorder = Source.bPulse || Source.bSelected || Source.bSearchHighlighted;
		if (DrawBorder)
		{
			FLinearColor BorderColor = SelectBorderColor(Source.bPulse, Source.bSelected, Source.bSearchHighlighted, NetRecvColor);
			DrawContext.DrawBox(X-1, NRY-1, W+2, NRH+2, WhiteBrush, BorderColor);
		}

		DrawContext.DrawBox(X, NRY, W, NRH, WhiteBrush, NetRecvColor);
	};
	
	for (const FSimulationActorGroup& ActorGroup : View.ActorGroups)
	{
		if (ActorGroup.Y > MinY)
		{
			DrawContext.DrawText(0, ActorGroup.Y, ActorGroup.DisplayString, Font, TextColor);
			DrawContext.LayerId++;
		}

		for (const FSimulationTrack& Track : ActorGroup.SimulationTracks)
		{
			if (Track.Y > MinY)
			{
				DrawContext.DrawText(0, Track.Y, Track.DisplayString, Font, TextColor);
				DrawContext.LayerId++;
			}

			// Orphan recvs first so they draw behind everything	
			if (Track.SubTracks.Num() > 0 && Track.SubTracks[0].Y > MinY)
			{
				const float Y = Track.SubTracks[0].Y;
				const float H = Track.SubTracks[0].H;
				for (const FSimulationTrack::FSubTrack::FDrawNetRecv& NetRecv : Track.OrphanedDrawNetRecvList)
				{
					DrawNetRecv(NetRecv, Y, H);
				}
			}

			for (const FSimulationTrack::FSubTrack& SubTrack : Track.SubTracks)
			{
				const float Y = SubTrack.Y;
				const float H = SubTrack.H;

				for (const FSimulationTrack::FSubTrack::FDrawTick& Tick : SubTrack.DrawTickList)
				{
					DrawTick(Tick, Y, H);
				}
				
				for (const FSimulationTrack::FSubTrack::FDrawNetRecv& NetRecv : SubTrack.DrawNetRecvList)
				{
					DrawNetRecv(NetRecv, Y, H);
				}
			}
		}
	}
}

// --------------------------------------------------------------------

SNPSimFrameView::SNPSimFrameView()
{

}

SNPSimFrameView::~SNPSimFrameView()
{

}

void SNPSimFrameView::Construct(const FArguments& InArgs, TSharedPtr<SNPWindow> InNetworkPredictionWindow)
{
	NetworkPredictionWindow = InNetworkPredictionWindow;
	InsightsModule = &FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");

	ChildSlot
	[
		SNew(SOverlay)
		.Visibility(EVisibility::SelfHitTestInvisible)

		+ SOverlay::Slot()
		.VAlign(VAlign_Top)
		.Padding(FMargin(0, 0, 0, 0))
		[
			SAssignNew(HorizontalScrollBar, SScrollBar)
			.Orientation(Orient_Horizontal)
			.AlwaysShowScrollbar(false)
			.Visibility(EVisibility::Visible)
			.Thickness(FVector2D(5.0f, 5.0f))
			.RenderOpacity(0.75)
			.OnUserScrolled(this, &SNPSimFrameView::HorizontalScrollBar_OnUserScrolled)
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.Padding(FMargin(0, 0, 0, 0))
		[
			SAssignNew(VerticalScrollBar, SScrollBar)
			.Orientation(Orient_Vertical)
			.AlwaysShowScrollbar(false)
			.Visibility(EVisibility::Visible)
			.Thickness(FVector2D(5.0f, 5.0f))
			.RenderOpacity(0.75)
			.OnUserScrolled(this, &SNPSimFrameView::VerticalScrollBar_OnUserScrolled)
		]
	];

	UpdateHorizontalScrollBar();
	UpdateVerticalScrollBar();

	Reset();

	NetworkPredictionWindow->OnFilteredDataCollectionChange.AddLambda([this](const FFilteredDataCollection& FilteredData)
	{
		bIsStateDirty = true; 
		if (bAutoScroll)
		{
			bAutoScrollDirty = true;
		}
	});
}

float SNPSimFrameView::PulsePCT;

void SNPSimFrameView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	PulseAccumulator += InDeltaTime;
	const float SinIn = PulseAccumulator * (2.f * PI);
	PulsePCT = (FMath::Sin(SinIn) + 1.f) / 2.f;
	if (PulseAccumulator > 100000.f)
	{
		PulseAccumulator = 0.f;
	}

	if (ThisGeometry != AllottedGeometry || bIsViewportDirty)
	{
		bIsViewportDirty = false;
		const float ViewWidth = AllottedGeometry.GetLocalSize().X;
		const float ViewHeight = AllottedGeometry.GetLocalSize().Y;
		Viewport.SetSize(ViewWidth, ViewHeight);
		bIsDrawDirty = true;
	}

	ThisGeometry = AllottedGeometry;

	FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	if (!bIsScrolling)
	{
		// Elastic snap to horizontal limits.
		if(ViewportX.UpdatePosWithinLimits())
		{
			bIsDrawDirty = true;
		}

		/*
		if (Viewport.GetVerticalAxisViewport().UpdatePosWithinLimits())
		{
			bIsDrawDirty = true;
		}*/
	}

	FNPAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

	float MinPos = 0.f;
	float MaxPos = FMath::Max<float>(0.f, Viewport.GetVerticalAxisViewport().GetMaxPos() - ViewportY.GetSize());

	if (Viewport.GetVerticalAxisViewport().EnforceScrollLimits(MinPos, MaxPos, 0.f))
	{
		bIsDrawDirty = true;
	}

	/*
	uint64 Time = FPlatformTime::Cycles64();
	if (Time > AnalysisSyncNextTimestamp)
	{
		const uint64 WaitTime = static_cast<uint64>(0.1 / FPlatformTime::GetSecondsPerCycle64()); // 100ms
		AnalysisSyncNextTimestamp = Time + WaitTime;

		TSharedPtr<const TraceServices::IAnalysisSession> Session = FInsightsManager::Get()->GetSession();
		if (Session.IsValid())
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
			const TraceServices::INetProfilerProvider& NetProfilerProvider = TraceServices::ReadNetProfilerProvider(*Session.Get());

			const uint32 NewConnectionChangeCount = NetProfilerProvider.GetConnectionChangeCount();
			if (NewConnectionChangeCount != ConnectionChangeCount)
			{
				ConnectionChangeCount = NewConnectionChangeCount;
				bIsStateDirty = true;
			}
		}
	}
	*/

	if (bIsStateDirty)
	{
		bIsStateDirty = false;
		UpdateState();
		bIsDrawDirty = true;
	}

	if (bIsDrawDirty || bAutoScrollDirty)
	{
		UpdateDraw();
		bIsDrawDirty = false;
	}
}

void SNPSimFrameView::UpdateState()
{
	check(InsightsModule);

	//FStopwatch Stopwatch;
	//Stopwatch.Start();
	
	FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.SetMinMaxInterval(0, 0);

	const FNPAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
	
	// ------------------------------------------------------------------------------------------------------------------
	//	Filter and order what we want to draw here. This will only get called when our provided data changes
	// ------------------------------------------------------------------------------------------------------------------
	
	SimulationFrameView.Reset();

	TArray<FSimulationActorGroup>& ActorGroups = SimulationFrameView.ActorGroups;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsModule->GetAnalysisSession();
	if (Session.IsValid() && NetworkPredictionWindow.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const INetworkPredictionProvider* NetworkPredictionProvider = ReadNetworkPredictionProvider(*Session.Get());
		if (NetworkPredictionProvider)
		{
			BuildSimulationView_ActorGroups(NetworkPredictionWindow->GetFilteredDataCollection());
		}
	}
}

// -----------------------------------------------------------------------------------------------------------
//	Resets and fills out SimulationFrameView: Creates and sorts ActorGroups that we want to draw
// -----------------------------------------------------------------------------------------------------------
void SNPSimFrameView::BuildSimulationView_ActorGroups(const FFilteredDataCollection& FilteredDataCollection)
{
	SimulationFrameView.Reset();
	
	TArray<FSimulationActorGroup>& ActorGroups = SimulationFrameView.ActorGroups;
	SimulationFrameView.HeadEngineFrame = FilteredDataCollection.LastEngineFrame;

	// -----------------------------------------------------------------------------------------------------------
	// Populate ActorGroups (grouping of simulations by their underlying ID, across all clients and server)
	// -----------------------------------------------------------------------------------------------------------
	for (const auto& SimView : FilteredDataCollection.Simulations)
	{
		const auto& SparseData = SimView->SparseData;
		if (ActorGroups.FindByPredicate([&](const FSimulationActorGroup& Existing) { return Existing.ID == SimView->ConstData.ID; }) == nullptr)
		{
			// ID not found in the ActorGroups, add it. Are there any AP simulations for this group?
			const bool bHasAutoProxy = FilteredDataCollection.Simulations.FindByPredicate([&](TSharedRef<FSimulationData::FRestrictedView>& SearchSimView) -> bool
			{
				return (SearchSimView->ConstData.ID == SimView->ConstData.ID) && (SearchSimView->SparseData->NetRole == ENP_NetRole::AutonomousProxy);
			}) != nullptr;

			ActorGroups.Emplace( FSimulationActorGroup{ SimView->ConstData.ID, SimView->ConstData.DebugName, SimView->ConstData.GroupName, bHasAutoProxy } );
		}
	}

	// Sort the actor group: the user can explicitly sort specific simulations. Otherwise bubble up AP-having sims, then by group ID.
	ActorGroups.Sort([this](const FSimulationActorGroup& A, const FSimulationActorGroup& B)
	{
		// Explicit sorting first
		int32 UserIndexA = UserSortedNetActors.IndexOfByKey(A.ID);
		int32 UserIndexB = UserSortedNetActors.IndexOfByKey(B.ID);
		if (UserIndexA != UserIndexB)
		{
			return UserIndexA > UserIndexB;
		}

		// Score them
		int32 ScoreA = A.bHasAutoProxy ? 100 : 0;
		int32 ScoreB = B.bHasAutoProxy ? 100 : 0;
		
		ScoreA += A.ID.SimID < B.ID.SimID ? 1 : -1;
		return ScoreA > ScoreB;
	});

	// -----------------------------------------------------------------------------------------------------------
	// Populate Simulation Tracks within the actor groups
	// -----------------------------------------------------------------------------------------------------------

	// The network-role order we want to display simulations within a group
	TArray<ENP_NetRole> TrackRoleOrder = {ENP_NetRole::Authority, ENP_NetRole::AutonomousProxy, ENP_NetRole::SimulatedProxy};

	for (FSimulationActorGroup& Group : ActorGroups)
	{
		for (ENP_NetRole Role : TrackRoleOrder)
		{
			for (int32 idx=0; idx < FilteredDataCollection.Simulations.Num(); ++idx)
			{
				const auto& SimView = FilteredDataCollection.Simulations[idx];
				const auto& SparseData = SimView->SparseData;

				// Find unused match on SimID and role
				if (SimView->ConstData.ID == Group.ID && SparseData->NetRole == Role)
				{
					Group.SimulationTracks.Emplace( FSimulationTrack{SimView});
					Group.MaxAllowedSimTime = FMath::Max(Group.MaxAllowedSimTime, SimView->GetMaxSimTime());
					Group.MaxEngineFrame = FMath::Max(Group.MaxEngineFrame, SimView->GetMaxEngineFrame());

					SimulationFrameView.HeadEngineFrame = FMath::Max(SimulationFrameView.HeadEngineFrame, Group.MaxEngineFrame);
				}
			}
		}
	}


	// -----------------------------------------------------------------------------------------------------------
	// Calculate offsets
	//
	//	All we are doing here is setting FSimulationActorGroup::OffsetSimTimeMS to offset independent ticking times so 
	//	that everything lines up in the UI.
	//
	//	Problem: independent ticking simulations' SimTime do not line up with each other or with the fixed ticking sims.
	//	(They tick on their own based on client frame rate. Their "total sim time" is local to the sim only. Our job here
	//	is to correlate these sim times and offset these actor groups (via FSimulationActorGroup::OffsetSimTimeMS) so that
	//	the tracks line up in the UI.
	//
	// -----------------------------------------------------------------------------------------------------------

	auto IsGroupAPIndependentTick = [](FSimulationActorGroup& Group)
	{
		return Group.SimulationTracks.Num() == 0 ? false :
			Group.SimulationTracks[0].View->SparseData->TickingPolicy == ENP_TickingPolicy::Independent && Group.bHasAutoProxy;
	};

	SimulationFrameView.PresentableTimeMS = 0;
	
	for (FSimulationActorGroup& Group : ActorGroups)
	{
		Group.OffsetSimTimeMS = 0;

		if (IsGroupAPIndependentTick(Group))
		{
			// AP controlled independent ticking sim requires an offset.
			// Find the AuthorityTrack
			const FSimulationTrack* AuthorityTrack = nullptr;
			uint64 AuthTrackMin = 0;
			uint64 AuthTrackMax = 0;

			for (const FSimulationTrack& Track : Group.SimulationTracks)
			{
				if (Track.View->SparseData->NetRole == ENP_NetRole::Authority)
				{
					AuthorityTrack = &Track;
					AuthTrackMin = Track.View->Ticks.GetFirst().EngineFrame;
					AuthTrackMax = Track.View->Ticks.GetLast().EngineFrame;
					break;
				}
			}

			if (!AuthorityTrack)
			{
				continue;
			}

			// function that aligns the authoritytrack to Track
			auto AttemptOffsetGroupByTrack = [&Group, AuthorityTrack, AuthTrackMin, AuthTrackMax](const FSimulationTrack& Track)
			{
				if (Track.View->SparseData->NetRole == ENP_NetRole::Authority)
				{
					const uint64 ThisTrackMin = Track.View->Ticks.GetFirst().EngineFrame;
					const uint64 ThisTrackMax = Track.View->Ticks.GetLast().EngineFrame;

					// Do they overlap at all?
					if (ThisTrackMax < AuthTrackMin || ThisTrackMin > AuthTrackMax)
					{
						return false;
					}

					auto FindIdx = [](const TRestrictedPageArrayView<FSimulationData::FTick>& TickStateView, uint64 EngineFrame)
					{
						const FSimulationData::FTick* Found = nullptr;
						for (auto It = TickStateView.GetIteratorFromEnd(); It; --It)
						{
							if (It->EngineFrame <= EngineFrame)
							{
								Found = &*It;
								break;
							}
						}

						return Found;
					};

					// Find common EngineFrame data
					const uint64 Max = FMath::Min(AuthTrackMax, ThisTrackMax);
					const FSimulationData::FTick* BaseState = FindIdx(Track.View->Ticks, Max);
					const FSimulationData::FTick* CurrentState = FindIdx(AuthorityTrack->View->Ticks, Max);

					// Calculate Delta
					const FSimTime BaseSimMS = BaseState->EndMS;
					const FSimTime CurrentSimMS = CurrentState->EndMS;
					const FSimTime Delta = BaseSimMS - CurrentSimMS;

					// Set the offset
					Group.OffsetSimTimeMS = Delta;
					
					return true;
				}
				return false;
			};
			
			// Find a track in another group to align to.
			//	-Ideally it is a non AP independent tick sim
			//	-If that doesn't exist, we use the first AP sim
			bool bResolvedOffset = false;
			for (FSimulationActorGroup& FixedTickGroup : ActorGroups)
			{
				if (IsGroupAPIndependentTick(FixedTickGroup) == false)
				{
					// Calculate SimTime offset based on this group
					for (const FSimulationTrack& Track : FixedTickGroup.SimulationTracks)
					{
						if (AttemptOffsetGroupByTrack(Track))
						{
							bResolvedOffset = true;
							break;
						}
					}

					if (bResolvedOffset)
					{
						break;
					}
				}
			}

			if (!bResolvedOffset)
			{
				if (ActorGroups.Num() > 0 && &ActorGroups[0] != &Group)
				{
					for (const FSimulationTrack& Track : ActorGroups[0].SimulationTracks)
					{
						if (AttemptOffsetGroupByTrack(Track))
						{
							bResolvedOffset = true;
							break;
						}
					}
				}
			}
		}

		for (const FSimulationTrack& Track : Group.SimulationTracks)
		{
			SimulationFrameView.PresentableTimeMS = FMath::Max(SimulationFrameView.PresentableTimeMS, Track.View->GetMaxSimTime());			
		}
	}

	// Calc ViewportMaxSimTimeMS
	for (FSimulationActorGroup& Group : ActorGroups)
	{
		ViewportMaxSimTimeMS = FMath::Max(ViewportMaxSimTimeMS, Group.MaxAllowedSimTime + Group.OffsetSimTimeMS);
	}


	BuildSimulationView_Tracks();
}

// -----------------------------------------------------------------------------------------------------------
//	Builds out the actual Simulation Tracks and SubTracks that each group will display.
//	Does everything up until we need screen space information
// -----------------------------------------------------------------------------------------------------------
void SNPSimFrameView::BuildSimulationView_Tracks()
{
	TArray<FSimulationActorGroup>& ActorGroups = SimulationFrameView.ActorGroups;
	const FSimulationData::FTick* SelectedTick = NetworkPredictionWindow->GetSelectedContent().SimTick;
	const FSimulationData::FNetSerializeRecv* SelectedNetRecv = NetworkPredictionWindow->GetSelectedContent().NetRecv;

	for (FSimulationActorGroup& Group : ActorGroups)
	{
		auto ProcessNetRecv = [&](const FSimulationData::FNetSerializeRecv& NetRecv, TArray<FSimulationTrack::FSubTrack::FNetRecvSource>& NetRecvSourceList, const FSimulationData::FRestrictedView& SimView)
		{
			if (NetRecv.EngineFrame > SimulationFrameView.HeadEngineFrame)
			{
				return;
			}

			const int32 NetMS = NetRecv.SimTimeMS + Group.OffsetSimTimeMS;

			const bool bPulseNetRecv = NetRecv.EngineFrame == SimulationFrameView.HeadEngineFrame;
			const bool bSelected = SelectedNetRecv == &NetRecv;
			const bool bSearchHighlighted = PerformSearch(NetRecv, SimView);

			NetRecvSourceList.Emplace(FSimulationTrack::FSubTrack::FNetRecvSource{NetRecv, NetMS, bPulseNetRecv, bSelected, bSearchHighlighted});
		};

		Group.DisplayString = FString::Printf(TEXT("%s [SimID: %d] %s"), *Group.DebugName, Group.ID.SimID, *LexToString(Group.MaxAllowedSimTime));

		for (FSimulationTrack& Track : Group.SimulationTracks)
		{
			const FSimulationData::FRestrictedView& SimView = Track.View.Get();
			const FSimulationData::FSparse& SparseData = SimView.SparseData.Get();

			Track.DisplayString = FString::Printf(TEXT("   %s"), LexToString(SparseData.NetRole));
			Track.SubTracks.Reset();

			FSimulationTrack::FSubTrack* SubTrack = &Track.SubTracks.Emplace_GetRef();
			uint64 PrevEngineFrame = 0;
			bool bPrevDesat = false;

			for (auto It = SimView.Ticks.GetIterator(); It; ++It)
			{
				const FSimulationData::FTick& Tick = *It;

				const FSimTime StartMS = Tick.StartMS + Group.OffsetSimTimeMS;
				const FSimTime EndMS = Tick.EndMS + Group.OffsetSimTimeMS;

				// -------------------------------------------------------------------
				//	Get the SubTrack that this tick should go on
				// -------------------------------------------------------------------

				const bool bFitsPrevSubTrack = SubTrack ? (SubTrack->PrevMS <= StartMS) : false;

				// Search for a new SubTrack if we don't fit on the last one or if we not in linear mode (linear mode = stay on the same subtrack unless forced off)
				if (!bFitsPrevSubTrack || !bLinearSimFrameView)
				{
					if (bCompactSimFrameView)
					{
						// Find first one that can fit us, starting from the top. Creates a zig zag style pattern
						SubTrack = Track.SubTracks.FindByPredicate([&StartMS](const FSimulationTrack::FSubTrack& ST)
						{
							return ST.PrevMS <= StartMS;
						});
					}
					else
					{
						// Check [0] first but then only consider tracks below us. Keeps a "cascade" style pattern
						auto CurrentSubTrack = SubTrack;
						SubTrack = nullptr;

						bool bPassedCurrent = false;
						for (int32 idx=0; idx < Track.SubTracks.Num(); ++idx)
						{
							if (&Track.SubTracks[idx] == CurrentSubTrack)
							{
								bPassedCurrent = true;
							}

							if (idx > 0 && !bPassedCurrent)
							{
								continue;
							}

							if (Track.SubTracks[idx].PrevMS <= StartMS)
							{
								SubTrack = &Track.SubTracks[idx];
								break;
							}
						}
					}
				}

				// No suitable track found, create a new one
				if (SubTrack == nullptr)
				{
					SubTrack = &Track.SubTracks.Emplace_GetRef();
				}

				SubTrack->PrevMS = EndMS;

				// -------------------------------------------------------------------
				//	Finalize the tick
				// -------------------------------------------------------------------

				// Toggle desat flag if we are on a new engine frame
				if (PrevEngineFrame != Tick.EngineFrame)
				{
					bPrevDesat = !bPrevDesat;
					PrevEngineFrame = Tick.EngineFrame;
				}

				// Calc FrameStatus and whether or not it should be pulsing
				ESimFrameStatus FrameStatus = ESimFrameStatus::Abandoned;
				uint64 PulseFrameNum = 0;
												
				if (SparseData.NetRole != ENP_NetRole::Authority)
				{
					if (Tick.TrashedEngineFrame > 0 && Tick.TrashedEngineFrame <= SimulationFrameView.HeadEngineFrame)
					{
						FrameStatus = ESimFrameStatus::Trashed;
						PulseFrameNum = Tick.TrashedEngineFrame;
					}
					else if (Tick.ConfirmedEngineFrame > 0 && Tick.ConfirmedEngineFrame <= SimulationFrameView.HeadEngineFrame)
					{
						FrameStatus = ESimFrameStatus::Confirmed;
						PulseFrameNum = Tick.ConfirmedEngineFrame;
					}
					else
					{
						FrameStatus = Tick.bRepredict ? ESimFrameStatus::Repredicted : ESimFrameStatus::Predicted;
						PulseFrameNum = Tick.EngineFrame;
					}							
				}
				else
				{
					if (Tick.bInputFault)
					{
						FrameStatus = ESimFrameStatus::Trashed;
						PulseFrameNum = Tick.EngineFrame;
					}
					else
					{
						FrameStatus = ESimFrameStatus::Confirmed;
						PulseFrameNum = Tick.EngineFrame;
					}
				}

				const bool bPulseFrame = (PulseFrameNum == SimulationFrameView.HeadEngineFrame);
				const bool bSelected = (&Tick == SelectedTick);
				const bool bIsSearchHighlighted = PerformSearch(Tick, SimView);
				
				SubTrack->TickSourceList.Emplace(FSimulationTrack::FSubTrack::FTickSource{Tick, StartMS, EndMS, bPrevDesat, bPulseFrame, bSelected, bIsSearchHighlighted, FrameStatus});
				
				// Process attached NetRecv
				if (Tick.StartNetRecv)
				{
					ensure(Tick.StartNetRecv->NextTick == &Tick);
					ensure(Tick.StartMS == Tick.StartNetRecv->SimTimeMS);
					ProcessNetRecv(*Tick.StartNetRecv, SubTrack->NetRecvSourceList, SimView);
				}
			}
			
			// Look for orphaned NetRecvs (they aren't attached to a simulation tick)
			for (auto It = SimView.NetRecv.GetIterator(); It; ++It)
			{
				const FSimulationData::FNetSerializeRecv& NetRecv = *It;
				if (NetRecv.NextTick == nullptr)
				{
					ProcessNetRecv(NetRecv, Track.OrphanedNetRecvSourceList, SimView);
				}
			}
		}
	}

	// Now that all tracks and subtracks are laid out on the X axis, calculate the "value Y" for everything

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FSlateFontInfo& Font = FCoreStyle::GetDefaultFontStyle("Regular", 8);
	const float MaxFontCharHeight = FontMeasureService->Measure(TEXT("!"), Font).Y;

	const float Y_Pad = 20.f;
	ViewportYMaxValue = 0.f;
	float Y = ViewportYContentStartValue;
	for (FSimulationActorGroup& Group : ActorGroups)
	{
		const float Y_Start = Y;
		Y += Y_Pad;
		Group.ValueY = Y;
		Y += MaxFontCharHeight;	// Account for Group.DisplayString

		for (FSimulationTrack& Track : Group.SimulationTracks)
		{
			Y += Y_Pad/2.f;
			Track.ValueY = Y;

			for (FSimulationTrack::FSubTrack& SubTrack : Track.SubTracks)
			{
				SubTrack.ValueY = Y;
				SubTrack.H = MaxFontCharHeight;
				Y += MaxFontCharHeight + Y_Pad;
			}
		}

		Group.H = Y - Y_Start;
	}

	ViewportYMaxValue = Y;
}

// -----------------------------------------------------------------------------------------------------------
//	Builds out the final lists of data used to actually draw stuff in OnPaint. This is the only update function
//	That should be looking at viewport or screenspace information.
// -----------------------------------------------------------------------------------------------------------
void SNPSimFrameView::UpdateDraw()
{
	// -------------------------------------------------------------------------------------------------------
	//	Adjust viewport before calculating our min/max draw values
	// -------------------------------------------------------------------------------------------------------

	FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	FNPAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

	// This viewport is capable of viewing the entire sim timeline
	ViewportX.SetMinMaxInterval(ViewportMinSimTimeMS, ViewportMaxSimTimeMS);
	ViewportY.SetMinMaxValueInterval(0.f, ViewportYMaxValue);

	if (bAutoScrollDirty)
	{
		// We want to center on the processed simulation time
		ViewportX.CenterOnValue(SimulationFrameView.PresentableTimeMS);
				
		// But that may put us within the "UpdatePosWithinLimits" limit which causes auto scrolling to happen in the main Tick function.
		float MinPos, MaxPos;
		ViewportX.GetScrollLimits(MinPos, MaxPos);
		ViewportX.EnforceScrollLimits(MinPos, MaxPos, 0); // 0 so that we don't lerp
				
		bAutoScrollDirty = false;
	}

	// -------------------------------------------------------------------------------------------------------

	TArray<FSimulationActorGroup>& ActorGroups = SimulationFrameView.ActorGroups;

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FSlateFontInfo& Font = FCoreStyle::GetDefaultFontStyle("Regular", 8);

	const float SampleW = Viewport.GetSampleWidth();
	const float ViewHeight = FMath::RoundToFloat(Viewport.GetHeight());
	const float H = Viewport.GetSubTrackHeight();
	
	const float MinWidthForEmbeddedText = FontMeasureService->Measure(TEXT("XXXXX"), Font).X; // Not great, would need to know highest output frame in all tracks which we currently don't cache anywhere
	const float SubTrackOffsetX = FontMeasureService->Measure(TEXT("   Autonomous  "), Font).X;
	
	const float X_Pad = MinWidthForEmbeddedText;

	// Create our draw list used by ::DrawCached

	const int32 MinDrawSimMS = FMath::Max(ViewportMinSimTimeMS, ViewportX.GetValueAtOffset(SubTrackOffsetX));
	const int32 MaxDrawSimMS = ViewportX.GetValueAtOffset(ViewportX.GetSize());

	for (FSimulationActorGroup& Group : ActorGroups)
	{
		Group.Y = ViewportY.GetOffsetForValue(Group.ValueY);

		for (FSimulationTrack& Track : Group.SimulationTracks)
		{
			auto ProcessNetRecvList = [&](TArray<FSimulationTrack::FSubTrack::FNetRecvSource>& NetRecvSourceList, TArray<FSimulationTrack::FSubTrack::FDrawNetRecv>& DrawNetRecvList)
			{
				for (const FSimulationTrack::FSubTrack::FNetRecvSource& NetRecv : NetRecvSourceList)
				{
					if (NetRecv.OffsetNetMS < MinDrawSimMS)
					{
						continue;
					}
					if (NetRecv.OffsetNetMS > MaxDrawSimMS)
					{
						break;
					}

					const float NetX = ViewportX.GetOffsetForValue(NetRecv.OffsetNetMS);
					DrawNetRecvList.Emplace( FSimulationTrack::FSubTrack::FDrawNetRecv{NetX, NetRecv} );
				}
			};

			Track.Y = ViewportY.GetOffsetForValue(Track.ValueY);

			const FSimulationData::FRestrictedView& SimView = Track.View.Get();
			const FSimulationData::FSparse& SparseData = SimView.SparseData.Get();

			for (FSimulationTrack::FSubTrack& SubTrack : Track.SubTracks)
			{
				SubTrack.DrawTickList.Reset();
				SubTrack.DrawNetRecvList.Reset();

				SubTrack.Y = ViewportY.GetOffsetForValue(SubTrack.ValueY);

				if (SubTrack.Y < ViewportYContentStartValue || SubTrack.Y > ViewportY.GetMaxPos())
				{
					continue;
				}

				for (const FSimulationTrack::FSubTrack::FTickSource& TickSource : SubTrack.TickSourceList)
				{
					if (TickSource.OffsetEndMS < MinDrawSimMS)
					{
						continue;
					}
					if (TickSource.OffsetStartMS > MaxDrawSimMS)
					{
						break;
					}

					const float StartX = FMath::Max(SubTrackOffsetX, ViewportX.GetOffsetForValue(TickSource.OffsetStartMS));
					const float EndX = FMath::Max(SubTrackOffsetX, ViewportX.GetOffsetForValue(TickSource.OffsetEndMS));

					const float W = EndX - StartX;
					if (W <= 0.f)
					{
						continue;
					}

					SubTrack.DrawTickList.Emplace( FSimulationTrack::FSubTrack::FDrawTick{StartX, W, TickSource} );
				}

				ProcessNetRecvList(SubTrack.NetRecvSourceList, SubTrack.DrawNetRecvList);
			}

			Track.OrphanedDrawNetRecvList.Reset();
			ProcessNetRecvList(Track.OrphanedNetRecvSourceList, Track.OrphanedDrawNetRecvList);
		}
	}
}

void SNPSimFrameView::SetAutoScroll(bool bIn)
{
	bAutoScroll = bIn;
	if (bIn)
	{
		bAutoScrollDirty = true;
	}
}
void SNPSimFrameView::SetAutoScrollDirty()
{
	bAutoScrollDirty = true;
}


////////////////////////////////////////////////////////////////////////////////////////////////////

class FSimFrameTrackDrawContext
{
public:
	explicit FSimFrameTrackDrawContext(const SNPSimFrameView* InSimFrameView, FDrawContext& InDrawContext, const FSimFrameViewDrawHelper& InHelper)
		: SimFrameView(InSimFrameView)
		, DrawContext(InDrawContext)
		, Helper(InHelper)
	{}

	const FSimFrameViewport& GetViewport() const { return SimFrameView->GetViewport(); }
	const FVector2D& GetMousePosition() const { return SimFrameView->GetMousePosition(); }
	
	/*
	const TSharedPtr<const ITimingEvent> GetHoveredEvent() const { return TimingView->GetHoveredEvent(); }
	const TSharedPtr<const ITimingEvent> GetSelectedEvent() const { return TimingView->GetSelectedEvent(); }
	const TSharedPtr<ITimingEventFilter> GetEventFilter() const { return TimingView->GetEventFilter(); }
	*/
	FDrawContext& GetDrawContext() const { return DrawContext; }
	
	const FSimFrameViewDrawHelper& GetHelper() const { return Helper; }

public:
	const SNPSimFrameView* SimFrameView;
	FDrawContext& DrawContext;
	const FSimFrameViewDrawHelper& Helper;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 SNPSimFrameView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	FDrawContext DrawContext(AllottedGeometry, MyCullingRect, InWidgetStyle, DrawEffects, OutDrawElements, LayerId);

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FSlateFontInfo SummaryFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush(TEXT("WhiteBrush")); //FInsightsStyle::Get().GetBrush("WhiteBrush");

	const float ViewWidth = AllottedGeometry.Size.X;
	const float ViewHeight = AllottedGeometry.Size.Y;


	{
		FSimFrameViewDrawHelper Helper(DrawContext, Viewport);
		Helper.DrawBackground(SimulationFrameView.PresentableTimeMS);

		Helper.DrawCached(SimulationFrameView, ViewportYContentStartValue);

		DrawHorizontalAxisGrid(DrawContext, WhiteBrush, SummaryFont);



		if (TooltipOpacity > 0.f)
		{
			FString Text;
			int NumLines = 0;

			if (HoverView.Tick)
			{
				Text += FString::Format(TEXT("Simulation Tick:\n"
													"Simulation Frame {0}\n"
													"Start Simulation MS: {1}\n"
													"End Simulation MS: {2}\n"
													"Delta Simulation MS: {3}\n"
													"Repredict: {4}\n"
													"Input Fault: {5}\n"
													"Confirmed Engine Frame: {6}\n"
													"Trashed Engine Frame: {7}\n"
													"GFrameNumber: {8}\n\n"),
					{
						HoverView.Tick->OutputFrame,
						HoverView.Tick->StartMS,
						HoverView.Tick->EndMS,
						(HoverView.Tick->EndMS - HoverView.Tick->StartMS),
						HoverView.Tick->bRepredict,
						HoverView.Tick->bInputFault,
						HoverView.Tick->ConfirmedEngineFrame,
						HoverView.Tick->TrashedEngineFrame,
						HoverView.Tick->EngineFrame
					});

				NumLines += 10;
			}

			if (HoverView.NetRecv)
			{
				

				Text += FString::Format(TEXT("Net Receive:\n"
													"SimFrame: {0}\n"
													"SimTime MS: {1}\n"
													"Status: {2}\n"
													"GFrameNumber: {3}\n"),
					{
						HoverView.NetRecv->Frame,
						HoverView.NetRecv->SimTimeMS,
						LexToString(HoverView.NetRecv->Status),						
						HoverView.NetRecv->EngineFrame
					});

				NumLines += 6;
			}

			

			FVector2D TextSize = FontMeasureService->Measure(Text, SummaryFont);

			const float DX = 2.0f;
			const float W2 = TextSize.X / 2 + DX;

			const FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
			float X1 = HoverView.X;
			float CX = X1 + FMath::RoundToFloat(Viewport.GetSampleWidth() / 2);
			
			if (CX + W2 > ViewportX.GetSize())
			{
				CX = FMath::RoundToFloat(ViewportX.GetSize() - W2);
			}
			if (CX - W2 < 0)
			{
				CX = W2;
			}
			
			const float H = 2.0f + 13.0f * NumLines;
			const float Y = FMath::Max(HoverView.Y - H, 20.f);


			if (MousePosition.Y > Y && MousePosition.Y < (Y + H))
			{
				CX -= (W2);
				if ((CX-W2) < 0.f)
				{
					CX += (W2*3.f);//+ 10.f;
				}
			}



			DrawContext.DrawBox(CX - W2, Y, 2 * W2, H, WhiteBrush, FLinearColor(0.7f, 0.7f, 0.7f, TooltipOpacity));
			DrawContext.LayerId++;
			DrawContext.DrawText(CX - W2 + DX, Y + 1.0f, Text, SummaryFont, FLinearColor(0.0, 0.0, 0.0, TooltipOpacity));
			DrawContext.LayerId++;
		}
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled());
}

void SNPSimFrameView::DrawHorizontalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font) const
{
	const FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();

	const float RoundedViewWidth = FMath::RoundToFloat(ViewportX.GetSize());

	constexpr float MinDX = 100.0f; // min horizontal distance between vertical grid lines

	const int32 LeftIndex = ViewportX.GetValueAtOffset(0.0f);
	const int32 GridIndex = ViewportX.GetValueAtOffset(MinDX);
	const int32 RightIndex = ViewportX.GetValueAtOffset(RoundedViewWidth);
	const int32 Delta = GridIndex - LeftIndex;

	if (Delta > 0)
	{
		// Compute rounding based on magnitude of visible range of samples (Delta).
		int32 Power10 = 1;
		int32 Delta10 = Delta;
		while (Delta10 > 0)
		{
			Delta10 /= 10;
			Power10 *= 10;
		}
		if (Power10 >= 100)
		{
			Power10 /= 100;
		}
		else
		{
			Power10 = 1;
		}

		const int32 Grid = ((Delta + Power10 - 1) / Power10) * Power10; // next value divisible with a multiple of 10

		// Skip grid lines for negative indices.
		double StartIndex = ((LeftIndex + Grid - 1) / Grid) * Grid;
		while (StartIndex < 0)
		{
			StartIndex += Grid;
		}

		const float ViewHeight = Viewport.GetHeight();

		const FLinearColor GridColor(0.0f, 0.0f, 0.0f, 0.1f);
		//const FLinearColor TextBgColor(0.05f, 0.05f, 0.05f, 1.0f);
		//const FLinearColor TextColor(1.0f, 1.0f, 1.0f, 1.0f);
		const FLinearColor TopTextColor(1.0f, 1.0f, 1.0f, 0.7f);

		//const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

		for (int32 Index = StartIndex; Index < RightIndex; Index += Grid)
		{
			const float X = FMath::RoundToFloat(ViewportX.GetOffsetForValue(Index));

			// Draw vertical grid line.
			DrawContext.DrawBox(X, 0, 1, ViewHeight, Brush, GridColor);

			const FString Text = FText::AsNumber(Index).ToString();
			//const FVector2D TextSize = FontMeasureService->Measure(Text, Font);
			//constexpr float TextH = 14.0f;

			// Draw background for index text.
			//DrawContext.DrawBox(X, ViewHeight - TextH, TextSize.X + 4.0f, TextH, Brush, TextBgColor);

			// Draw index text.
			//DrawContext.DrawText(X + 2.0f, ViewHeight - TextH + 1.0f, Text, Font, TextColor);

			DrawContext.DrawText(X + 2.0f, 10.0f, Text, Font, TopTextColor);
		}
		DrawContext.LayerId++;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////


FReply SNPSimFrameView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePositionOnButtonDown = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	ViewportPosXOnButtonDown = Viewport.GetHorizontalAxisViewport().GetPos();
	ViewportPosYOnButtonDown = Viewport.GetVerticalAxisViewport().GetPos();

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsLMB_Pressed = true;

		// Capture mouse.
		Reply = FReply::Handled().CaptureMouse(SharedThis(this));

		FMouseOverInfo MouseOver = GetMouseOverInfo(MousePositionOnButtonDown);

		if (MouseOver.DrawSimTick || MouseOver.DrawNetRecv)
		{
			const FSimulationData::FTick* ClickedTick = MouseOver.DrawSimTick ? &MouseOver.DrawSimTick->Source.Tick : nullptr;
			const FSimulationData::FNetSerializeRecv* ClickedMouseRecv = MouseOver.GetNetRecv(NetworkPredictionWindow->GetFilteredDataCollection().LastEngineFrame);
			NetworkPredictionWindow->NotifySimContentClicked(FSimContentsView { MouseOver.Track->View, 
																				ClickedTick, 
																				ClickedMouseRecv });
			bIsStateDirty = true;
		}
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bIsRMB_Pressed = true;

		// Capture mouse, so we can scroll outside this widget.
		Reply = FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton)
	{
		NetworkPredictionWindow->JumpPreviousViewedEngineFrame();
		Reply = FReply::Handled();
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton2)
	{
		NetworkPredictionWindow->JumpNextViewedEngineFrame();
		Reply = FReply::Handled();
	}
	return Reply;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SNPSimFrameView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePositionOnButtonUp = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	const bool bIsValidForMouseClick = MousePositionOnButtonUp.Equals(MousePositionOnButtonDown, MOUSE_SNAP_DISTANCE);

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (bIsLMB_Pressed)
		{
			if (bIsScrolling)
			{
				bIsScrolling = false;
				CursorType = ECursorType::Default;
			}
			else if (bIsValidForMouseClick)
			{
				//SelectSampleAtMousePosition(MousePositionOnButtonUp.X, MousePositionOnButtonUp.Y, MouseEvent);
			}

			bIsLMB_Pressed = false;

			// Release the mouse.
			Reply = FReply::Handled().ReleaseMouseCapture();
		}
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (bIsRMB_Pressed)
		{
			if (bIsScrolling)
			{
				bIsScrolling = false;
				CursorType = ECursorType::Default;
			}
			else if (bIsValidForMouseClick)
			{
				FMouseOverInfo MouseOver = GetMouseOverInfo(MousePositionOnButtonDown);

				FMenuBuilder MenuBuilder(true, nullptr);
				const FSimulationData::FNetSerializeRecv* NetRecv = MouseOver.GetNetRecv(NetworkPredictionWindow->GetFilteredDataCollection().LastEngineFrame);

				if (MouseOver.Group)
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("SortGroupToTop", "Sort to top"), 
						LOCTEXT("SortGroupToTopTip", "Bring this Actor Group to the top of the Simulation Timeline View"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateSP(this, &SNPSimFrameView::SortActorGroupToTop, MouseOver.Group)),
						NAME_None,
						EUserInterfaceActionType::Button
					);
				}
				
				if (MouseOver.DrawSimTick)
				{
					FString Text = FString::Printf(TEXT("Sim Tick: Jump To %" UINT64_FMT), MouseOver.DrawSimTick->Source.Tick.EngineFrame);

					MenuBuilder.AddMenuEntry(
						FText::FromString(Text), 
						LOCTEXT("JumpToSimFrameEngineNumToolTip", "Jump to the Engine Frame that this tick was made on."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateSP(NetworkPredictionWindow.Get(), &SNPWindow::SetEngineFrame, MouseOver.DrawSimTick->Source.Tick.EngineFrame, true)),
						NAME_None,
						EUserInterfaceActionType::Button
					);
				}
					
				if (NetRecv)
				{
					FString Text = FString::Printf(TEXT("Net Recv: Jump To %" UINT64_FMT), NetRecv->EngineFrame);

					MenuBuilder.AddMenuEntry(
						FText::FromString(Text),
						LOCTEXT("JumpToNetRecvEngineNumToolTip", "Jump to the Engine Frame that this Net Receive was received on."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateSP(NetworkPredictionWindow.Get(), &SNPWindow::SetEngineFrame, NetRecv->EngineFrame, true)),
						NAME_None,
						EUserInterfaceActionType::Button
					);
				}

				TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();

				FWidgetPath EventPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
				const FVector2D ScreenSpacePosition = MouseEvent.GetScreenSpacePosition();
				FSlateApplication::Get().PushMenu(SharedThis(this), EventPath, MenuWidget, ScreenSpacePosition, FPopupTransitionEffect::ContextMenu);
			}

			bIsRMB_Pressed = false;

			// Release mouse as we no longer scroll.
			Reply = FReply::Handled().ReleaseMouseCapture();
		}
	}

	return Reply;
}

void SNPSimFrameView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{

}
void SNPSimFrameView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		bIsLMB_Pressed = false;
		bIsRMB_Pressed = false;

		HoverView.Reset();
		TooltipOpacity = 0.f;

		CursorType = ECursorType::Default;
	}
}

FReply SNPSimFrameView::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (MouseEvent.GetModifierKeys().IsShiftDown())
	{
		FNPAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();

		// Zoom in/out vertically.
		const float Delta = MouseEvent.GetWheelDelta();
		constexpr float ZoomStep = 0.25f; // as percent
		float ScaleY;

		if (Delta > 0)
		{
			ScaleY = ViewportY.GetScale() * FMath::Pow(1.0f + ZoomStep, Delta);
		}
		else
		{
			ScaleY = ViewportY.GetScale() * FMath::Pow(1.0f / (1.0f + ZoomStep), -Delta);
		}

		ViewportY.SetScale(ScaleY);
		//UpdateVerticalScrollBar();
	}
	else //if (MouseEvent.GetModifierKeys().IsControlDown())
	{
		// Zoom in/out horizontally.
		const float Delta = MouseEvent.GetWheelDelta();
		ZoomHorizontally(Delta, MousePosition.X);
	}

	return FReply::Handled();
}

FReply SNPSimFrameView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();

	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (!MouseEvent.GetCursorDelta().IsZero())
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) ||
			MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
		{
			if (HasMouseCapture())
			{
				if (!bIsScrolling)
				{
					bIsScrolling = true;
					CursorType = ECursorType::Hand;
					
					HoverView.Reset();
				}

				FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
				const float PosX = ViewportPosXOnButtonDown + (MousePositionOnButtonDown.X - MousePosition.X);
				ViewportX.ScrollAtValue(ViewportX.GetValueAtPos(PosX)); // align viewport position with sample
				UpdateHorizontalScrollBar();


				FNPAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
				const float PosY = ViewportPosYOnButtonDown + (MousePositionOnButtonDown.Y - MousePosition.Y);
				ViewportY.ScrollAtValue(ViewportY.GetValueAtPos(PosY));
				UpdateVerticalScrollBar();


				OnUserScroll();
			}
		}
		else
		{
			TooltipOpacity = 0.f;
			HoverView.Tick = nullptr;
			HoverView.NetRecv = nullptr;

			FMouseOverInfo MouseOver = GetMouseOverInfo(MousePosition);
			if (MouseOver.DrawNetRecv && ensure(MouseOver.Track))
			{
				HoverView.NetRecv = &MouseOver.DrawNetRecv->Source.NetRecv;
				HoverView.X = MouseOver.DrawNetRecv->X;
				HoverView.Y = MouseOver.Track->Y;
				TooltipOpacity = 1.f;
			}

			if (MouseOver.DrawSimTick && ensure(MouseOver.Track))
			{
				HoverView.Tick = &MouseOver.DrawSimTick->Source.Tick;
				HoverView.X = MouseOver.DrawSimTick->X;
				HoverView.Y = MouseOver.Track->Y;
				TooltipOpacity = 1.f;
			}
		}

		Reply = FReply::Handled();
	}

	return Reply;
}

SNPSimFrameView::FMouseOverInfo SNPSimFrameView::GetMouseOverInfo(FVector2D InMousePosition) const
{
	FMouseOverInfo Info;

	const float Y = InMousePosition.Y;
	const float X = InMousePosition.X;

	if (Y > Viewport.GetContentStartY())
	{
		auto GetMouseOverInfo = [&]()
		{
			for (const FSimulationActorGroup& Group : SimulationFrameView.ActorGroups)
			{
				if (Y >= Group.Y && Y < (Group.Y + Group.H))
				{
					Info.Group = &Group;
					for (const FSimulationTrack& SimTrack : Group.SimulationTracks)
					{
						auto FindNetRecv = [&](const TArray<FSimulationTrack::FSubTrack::FDrawNetRecv>& DrawNetRecvList)
						{
							for (const FSimulationTrack::FSubTrack::FDrawNetRecv& DrawNetRecv : DrawNetRecvList)
							{
								if (X >= DrawNetRecv.X && X < (DrawNetRecv.X + DrawNetRecv.W()))
								{
									Info.DrawNetRecv = &DrawNetRecv;
									return;
								}
							}
						};

						Info.Track = &SimTrack;
						for (const FSimulationTrack::FSubTrack& SubTrack : SimTrack.SubTracks)
						{
							if (Y >= SubTrack.Y && Y < (SubTrack.Y + SubTrack.H))
							{
								Info.SubTrack = &SubTrack;

								auto FindTick = [&]()
								{
									for (const FSimulationTrack::FSubTrack::FDrawTick& DrawTick : SubTrack.DrawTickList)
									{
										if (X >= DrawTick.X && X < (DrawTick.X + DrawTick.W))
										{
											Info.DrawSimTick = &DrawTick;
											return;
										}
									}
								};

								if (&SubTrack == &SimTrack.SubTracks[0])
								{
									FindNetRecv(SimTrack.OrphanedDrawNetRecvList);
								}

								FindNetRecv(SubTrack.DrawNetRecvList);
								FindTick();
								return;
							}
						}

					}
					return;
				}
			}
		};
				
		GetMouseOverInfo();
	}

	return Info;
}

void SNPSimFrameView::Reset()
{
	Viewport.Reset();
	FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.SetScaleLimits(1.f, 16.0f); // 1 [sample/px] to 16 [px/sample]
	ViewportX.SetScale(5.0f);
	FNPAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
	bIsViewportDirty = true;
	bIsStateDirty = true;
}

void SNPSimFrameView::HorizontalScrollBar_OnUserScrolled(float ScrollOffset)
{
	FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.OnUserScrolled(HorizontalScrollBar, ScrollOffset);
	OnUserScroll();
}

void SNPSimFrameView::UpdateHorizontalScrollBar()
{
	FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.UpdateScrollBar(HorizontalScrollBar);
}

void SNPSimFrameView::VerticalScrollBar_OnUserScrolled(float ScrollOffset)
{
	FNPAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
	ViewportY.OnUserScrolled(VerticalScrollBar, ScrollOffset);
	OnUserScroll();
}

void SNPSimFrameView::UpdateVerticalScrollBar()
{
	FNPAxisViewportDouble& ViewportY = Viewport.GetVerticalAxisViewport();
	ViewportY.UpdateScrollBar(VerticalScrollBar);
}

void SNPSimFrameView::ZoomHorizontally(const float Delta, const float X)
{
	FNPAxisViewportInt32& ViewportX = Viewport.GetHorizontalAxisViewport();
	ViewportX.RelativeZoomWithFixedOffset(Delta, X);
	ViewportX.ScrollAtValue(ViewportX.GetValueAtPos(ViewportX.GetPos())); // align viewport position with sample
	UpdateHorizontalScrollBar();
	bIsDrawDirty = true;
}

void SNPSimFrameView::OnUserScroll()
{
	bAutoScroll = false;
	bIsDrawDirty = true;
}

void SNPSimFrameView::OnGetOptionsMenu(FMenuBuilder& Builder)
{
	Builder.BeginSection(NAME_None, LOCTEXT("SimulationFrameViewLabel", "Simulation Frame View"));
	{
		Builder.AddMenuEntry(
			LOCTEXT("CompactViewLabel","Compact View"), 
			LOCTEXT("CompactViewLabelToolTip", "Display Simulation Frames in a more compact fashion. Takes less vertical space but can make harder to read."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SNPSimFrameView::ToggleCompactSimFrameView),
					  FCanExecuteAction(),
					  FIsActionChecked::CreateSP(this, &SNPSimFrameView::CompactSimFrameView)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		Builder.AddMenuEntry(
			LOCTEXT("LinearViewLabel","Linear View"), 
			LOCTEXT("LinearViewLabelToolTip", "Display Simulation Frames in contingous linear series when possible. Can take more vertical space but is generally easier to read."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SNPSimFrameView::ToggleLinearSimFrameview),
					  FCanExecuteAction(),
					  FIsActionChecked::CreateSP(this, &SNPSimFrameView::LinearSimFrameView)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	Builder.EndSection();

	Builder.BeginSection(NAME_None, LOCTEXT("SimulationFrameContentViewLabel", "Simulation Frame Contents"));
	{
		Builder.AddMenuEntry(
			LOCTEXT("FrameViewLabel","Frame Number"), 
			LOCTEXT("FrameViewLabelToolTip", "Display Simulation Frames number in the tick boxes."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SNPSimFrameView::SetFrameContentView_FrameNumber),
					  FCanExecuteAction(),
					  FIsActionChecked::CreateSP(this, &SNPSimFrameView::IsFrameContentView_FrameNumber)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		Builder.AddMenuEntry(
			LOCTEXT("BufferedInputCmdsViewLabel","Buffered InputCmds"), 
			LOCTEXT("BufferedInputCmdsViewLabelToolTip", "Display number of buffered InputCmds in the tick boxes."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SNPSimFrameView::SetFrameContentView_NumBufferdInputCmds),
					  FCanExecuteAction(),
					  FIsActionChecked::CreateSP(this, &SNPSimFrameView::IsFrameContentView_BufferedInputCmds)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	Builder.EndSection();
}

void SNPSimFrameView::SearchUserData(const FText& InFilterText)
{
	UserStateSearchString = InFilterText.ToString();
	bIsStateDirty = true;
}

bool SNPSimFrameView::PerformSearch(const FSimulationData::FTick& Tick, const FSimulationData::FRestrictedView& SimView)
{
	if (UserStateSearchString.IsEmpty())
	{
		return false;
	}

	TArray<const FSimulationData::FUserState*, TInlineAllocator<5>> UserStates;

	const int32 InputFrame = Tick.OutputFrame - 1;
	const int32 OutputFrame = Tick.OutputFrame;
	const uint64 EngineFrame = Tick.EngineFrame;

	const uint8 Mask = (uint8)ENP_UserStateSource::NetRecv;

	UserStates.Add(SimView.UserData.Get(ENP_UserState::Input, InputFrame, EngineFrame, Mask));

	UserStates.Add(SimView.UserData.Get(ENP_UserState::Sync, InputFrame, EngineFrame, Mask));
	UserStates.Add(SimView.UserData.Get(ENP_UserState::Aux, InputFrame, EngineFrame, Mask));
	UserStates.Add(SimView.UserData.Get(ENP_UserState::Physics, InputFrame, EngineFrame, Mask));

	UserStates.Add(SimView.UserData.Get(ENP_UserState::Sync, OutputFrame, EngineFrame, Mask));
	UserStates.Add(SimView.UserData.Get(ENP_UserState::Aux, OutputFrame, EngineFrame, Mask));
	UserStates.Add(SimView.UserData.Get(ENP_UserState::Physics, OutputFrame, EngineFrame, Mask));
	
	return PerformSearchInternal(UserStates);
}

bool SNPSimFrameView::PerformSearch(const FSimulationData::FNetSerializeRecv& NetRecv, const FSimulationData::FRestrictedView& SimView)
{
	if (UserStateSearchString.IsEmpty())
	{
		return false;
	}

	TArray<const FSimulationData::FUserState*, TInlineAllocator<3>> UserStates;

	const int32 SimFrame = NetRecv.Frame;
	const uint64 EngineFrame = NetRecv.EngineFrame;

	uint8 Mask = ~((uint8)ENP_UserStateSource::NetRecv | (uint8)ENP_UserStateSource::NetRecvCommit);
	UserStates.Add(SimView.UserData.Get(ENP_UserState::Input, SimFrame, EngineFrame, Mask));
	UserStates.Add(SimView.UserData.Get(ENP_UserState::Sync, SimFrame, EngineFrame, Mask));
	UserStates.Add(SimView.UserData.Get(ENP_UserState::Aux, SimFrame, EngineFrame, Mask));

	return PerformSearchInternal(UserStates);
}

bool SNPSimFrameView::PerformSearchInternal(const TArrayView<const FSimulationData::FUserState* const>& UserStates)
{
	for (const FSimulationData::FUserState* UserState : UserStates)
	{
		if (UserState && FCString::Stristr(UserState->UserStr, *UserStateSearchString) != nullptr)
		{
			return true;
		}
	}
	return false;
}

void SNPSimFrameView::SortActorGroupToTop(const FSimulationActorGroup* Group)
{
	UserSortedNetActors.Remove(Group->ID);
	UserSortedNetActors.Push(Group->ID);
	bIsStateDirty = true;
}

#undef LOCTEXT_NAMESPACE
