// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/SlateFontInfo.h"
#include "NPAxisViewportInt32.h"
#include "NPAxisViewportDouble.h"
#include "INetworkPredictionProvider.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
struct FDrawContext;

enum class ESimFrameStatus : uint8
{
	Predicted,	// A predicted frame that has not been rolled back or confirmed in any way
	Repredicted,// Sim was rolled back to a point prior to this frame, and we repredicted this frame to *catch back up* to where we were
	Confirmed,	// We received authoratative state > that this frame and didn't rollback/reconcile
	Trashed,	// Frame that was thrown out: we rolled the sim back and ticked Repredicted frames past this one.
	Abandoned,	// The sim was skipped ahead by receiving authoritative state: this frame was never used as input into the next frame
};

// ---------------------------------------------------------------------------------------------------------------------------------
//
//	[ActorName, Role, NetGUID, Simulation Group/Type]											} FSimulationActorGroup[0]
//	--------------------------------------------------------------------------------------------------------------------------------
//	  | Auto Proxy |	******************PP													} SubTrack[0]  \ SimulationTracks[0]
//	  |	           |                !!!!!!														} SubTrack[1]  /
//	  ------------------------------------------------------------------------------------------------------------------------------
//	  | Authority  |    *******************														} SubTrack[0] -> SimulationTracks[1]
//	  ------------------------------------------------------------------------------------------------------------------------------
//	  | Sim Proxy  |    *******************														} SubTrack[0] -> SimulationTracks[2]
// ---------------------------------------------------------------------------------------------------------------------------------

struct FSimulationTrack
{
	// Processed Data about subtracks: this will indicate overlapping simulation steps
	// essentially, how we increase the Y size of the track in the UI
	struct FSubTrack
	{
		struct FTickSource
		{
			const FSimulationData::FTick& Tick;
			const FSimTime OffsetStartMS;
			const FSimTime OffsetEndMS;
			const bool bDesaturate;
			const bool bPulse;
			const bool bSelected;
			const bool bSearchHighlighted;
			const ESimFrameStatus Status;
		};

		struct FNetRecvSource
		{
			const FSimulationData::FNetSerializeRecv& NetRecv;
			const FSimTime OffsetNetMS;
			const bool bPulse;
			const bool bSelected;
			const bool bSearchHighlighted;
		};

		struct FDrawTick
		{
			const float X;
			const float W;
			const FTickSource& Source;
		};

		struct FDrawNetRecv
		{
			static constexpr float W() { return 3.f; }

			float X;
			const FNetRecvSource& Source;
		};

		// All ticks that belong to this track. These are not culled wrt the viewport
		TArray<FTickSource> TickSourceList;
		TArray<FNetRecvSource> NetRecvSourceList;
		int32 PrevMS = TNumericLimits<int32>::Min();

		// These are the actual draw lists that get flung over to ::DrawCached
		TArray<FDrawTick> DrawTickList;
		TArray<FDrawNetRecv> DrawNetRecvList;
		
		float ValueY=0.f; // untransformed by viewport/scrolling value
		float Y=0.f;
		float H=0.f;
	};

	TSharedRef<FSimulationData::FRestrictedView> View;

	float Y;
	float ValueY; // untransformed by viewport/scrolling value
	FString DisplayString;
	TArray<FSubTrack> SubTracks;

	// Orphaned Recvs don't map to a specific subtrack.
	TArray<FSubTrack::FNetRecvSource> OrphanedNetRecvSourceList;
	TArray<FSubTrack::FDrawNetRecv> OrphanedDrawNetRecvList;
};

struct FSimulationActorGroup
{
	FSimNetActorID ID;

	FString DebugName;
	FName GroupName;
	bool bHasAutoProxy; // Is there an auto proxy sim in this group

	uint64 MaxEngineFrame = 0; // Engine frame number of last simulation frame amonst all simulations for this actor
	FSimTime MaxAllowedSimTime = 0;
	
	FSimTime OffsetSimTimeMS; // How much to offset this group when drawing anything on the sim timeline

	TArray<FSimulationTrack> SimulationTracks;

	// Rendering Data. Only filled out if being drawn
	FString DisplayString;
	float Y;
	float H;

	float ValueY; // untransformed by viewport/scrolling value
};

enum ESimFrameContentType
{
	FrameNumber,
	NumBufferedInputCmds
};

// The overall processed data that the Simulation Frame Widget uses to draw
struct FSimulationFrameView
{
	void Reset()
	{
		HeadEngineFrame = 0;
		ActorGroups.Reset();
	}

	uint64 HeadEngineFrame = 0;
	FSimTime PresentableTimeMS = 0; // The "current" time that everything is anchored around
	TArray<FSimulationActorGroup> ActorGroups;

	ESimFrameContentType ContentType = ESimFrameContentType::FrameNumber;
};

// Cached data about what we are hovering over
struct FSimFrameHoverView
{
	const FSimulationData::FTick* Tick = nullptr;
	const FSimulationData::FNetSerializeRecv* NetRecv = nullptr;
	FSimTime X = 0;
	FSimTime Y = 0;

	void Reset()
	{
		Tick = nullptr;
		NetRecv = nullptr;
		X = 0;
		Y = 0;
	}
};

class SNPWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////

// Viewport that is simulation frame based (not real time!)
class FSimFrameViewport
{
private:
	static constexpr float SLATE_UNITS_TOLERANCE = 0.1f;

public:
	FSimFrameViewport()
	{
		Reset();
	}

	void Reset()
	{
		HorizontalAxisViewport.Reset();
		VerticalAxisViewport.Reset();
	}

	const FNPAxisViewportInt32& GetHorizontalAxisViewport() const { return HorizontalAxisViewport; }
	FNPAxisViewportInt32& GetHorizontalAxisViewport() { return HorizontalAxisViewport; }

	const FNPAxisViewportDouble& GetVerticalAxisViewport() const { return VerticalAxisViewport; }
	FNPAxisViewportDouble& GetVerticalAxisViewport() { return VerticalAxisViewport; }

	float GetWidth() const { return HorizontalAxisViewport.GetSize(); }
	float GetHeight() const { return VerticalAxisViewport.GetSize(); }

	bool SetSize(const float InWidth, const float InHeight)
	{
		const bool bWidthChanged = HorizontalAxisViewport.SetSize(InWidth);
		const bool bHeightChanged = VerticalAxisViewport.SetSize(InHeight);
		if (bWidthChanged || bHeightChanged)
		{
			OnSizeChanged();
			return true;
		}
		return false;
	}

	float GetSampleWidth() const { return HorizontalAxisViewport.GetSampleSize(); }
	int32 GetNumFramesPerSample() const { return HorizontalAxisViewport.GetNumSamplesPerPixel(); }
	int32 GetFirstFrameIndex() const { return HorizontalAxisViewport.GetValueAtOffset(0.0f); }

	float GetContentStartY() const { return 30.f; } //GetVerticalAxisViewport().GetSize() - 1.f; }
	float GetTrackPadding() const { return GetSubTrackHeight(); }
	float GetSubTrackHeight() const
	{
		const float BaselineY = FMath::RoundToFloat(VerticalAxisViewport.GetOffsetForValue(0.0));
		const float ValueY = FMath::RoundToFloat(VerticalAxisViewport.GetOffsetForValue(20.0));
		const float H = ValueY - BaselineY;
		return H;
	}
	

private:
	void OnSizeChanged()
	{
	}

private:
	FNPAxisViewportInt32 HorizontalAxisViewport;
	FNPAxisViewportDouble VerticalAxisViewport;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSimFrameViewDrawHelper
{
public:
	enum class EHighlightMode : uint32
	{
		Hovered = 1,
		Selected = 2,
		SelectedAndHovered = 3
	};

public:
	explicit FSimFrameViewDrawHelper(const FDrawContext& InDrawContext, const FSimFrameViewport& InViewport);

	/**
	 * Non-copyable
	 */
	FSimFrameViewDrawHelper(const FSimFrameViewDrawHelper&) = delete;
	FSimFrameViewDrawHelper& operator=(const FSimFrameViewDrawHelper&) = delete;

	void DrawBackground(FSimTime PresentableLine) const;
	void DrawCached(const FSimulationFrameView& View, float MinY) const;
	
	FLinearColor GetFramColorByStatus(ESimFrameStatus Status, const bool bDesaturate) const;
	FLinearColor GetRecvColorByStatus(ENetSerializeRecvStatus Status) const;
	
	/*
	void DrawSampleHighlight(const FNetworkPacketAggregatedSample& Sample, EHighlightMode Mode) const;
	void DrawSelection(int32 StartPacketIndex, int32 EndPacketIndex) const;
	

	static FLinearColor GetColorByStatus(TraceServices::ENetProfilerDeliveryStatus Status);

	int32 GetNumPackets() const { return NumPackets; }
	int32 GetNumDrawSamples() const { return NumDrawSamples; }
	*/

private:
	const FDrawContext& DrawContext;
	const FSimFrameViewport& Viewport;

	const FSlateBrush* WhiteBrush;
	const FSlateBrush* HoveredEventBorderBrush;
	const FSlateBrush* SelectedEventBorderBrush;
	const FSlateFontInfo SelectionFont;

	// Debug stats.
	mutable int32 NumFrames = 0;
	mutable int32 NumDrawSamples = 0;
};


////////////////////////////////////////////////////////////////////////////////////////////////////

class SNPSimFrameView : public SCompoundWidget
{
public:

	/** Number of pixels. */
	static constexpr float MOUSE_SNAP_DISTANCE = 2.0f;

	enum class ECursorType
	{
		Default,
		Arrow,
		Hand,
	};


	SNPSimFrameView();
	virtual ~SNPSimFrameView();

	SLATE_BEGIN_ARGS(SNPSimFrameView)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SNPWindow> InNetworkPredictionWindow);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void DrawHorizontalAxisGrid(FDrawContext& DrawContext, const FSlateBrush* Brush, const FSlateFontInfo& Font) const;

	const FSimFrameViewport& GetViewport() const { return Viewport; }
	const FVector2D& GetMousePosition() const { return MousePosition; }

	bool GetAutoScroll() const { return bAutoScroll; }
	void SetAutoScroll(bool bIn);
	void SetAutoScrollDirty();

	static float PulsePCT;

	void OnGetOptionsMenu(FMenuBuilder& Builder);

	void SearchUserData(const FText& InFilterText);

private:

	void Reset();
	void UpdateState();
	void BuildSimulationView_ActorGroups(const struct FFilteredDataCollection& FilteredDataCollection);
	void BuildSimulationView_Tracks();
	
	// 
	void UpdateDraw();

	void HorizontalScrollBar_OnUserScrolled(float ScrollOffset);
	void VerticalScrollBar_OnUserScrolled(float ScrollOffset);
	void UpdateHorizontalScrollBar();
	void UpdateVerticalScrollBar();
	void OnUserScroll();

	void ZoomHorizontally(const float Delta, const float X);	

	struct FMouseOverInfo
	{
		const FSimulationActorGroup* Group = nullptr;
		const FSimulationTrack* Track = nullptr;
		const FSimulationTrack::FSubTrack* SubTrack = nullptr;
		const FSimulationTrack::FSubTrack::FDrawNetRecv* DrawNetRecv = nullptr;
		const FSimulationTrack::FSubTrack::FDrawTick* DrawSimTick = nullptr;

		const FSimulationData::FNetSerializeRecv* GetNetRecv(uint64 MaxEngineFrame)
		{
			const FSimulationData::FNetSerializeRecv* Recv = DrawNetRecv ? &DrawNetRecv->Source.NetRecv : nullptr;
			if (!Recv && DrawSimTick)
			{
				if (DrawSimTick->Source.Tick.StartNetRecv && DrawSimTick->Source.Tick.StartNetRecv->EngineFrame <= MaxEngineFrame)
				{
					Recv = DrawSimTick->Source.Tick.StartNetRecv;
				}

			}
			return Recv;
		}
	};

	FMouseOverInfo GetMouseOverInfo(FVector2D InMousePosition) const;


	TSharedPtr<SNPWindow> NetworkPredictionWindow;

	/** The track's viewport. Encapsulates info about position and scale. */
	FSimFrameViewport Viewport;
	bool bIsViewportDirty = false;

	/** The slot that contains this widget's descendants.*/
	TSharedPtr<SScrollBar> HorizontalScrollBar;
	TSharedPtr<SScrollBar> VerticalScrollBar;

	bool bIsStateDirty = false;	// What we want to draw has changed: underlying analyzed data or filtering/settings.
	bool bIsDrawDirty = false;	// What we need to draw (may) have changed: scrolling, viewport changes etc can trigger this as well.
	
	bool bAutoScroll = true;			// User wants to auto scroll
	bool bAutoScrollDirty = false;		// We need to update our view to scroll to latest

	class IUnrealInsightsModule* InsightsModule;

	//////////////////////////////////////////////////
	// Panning and Zooming behaviors

	/** The current mouse position. */
	FVector2D MousePosition;

	/** Mouse position during the call on mouse button down. */
	FVector2D MousePositionOnButtonDown;
	float ViewportPosXOnButtonDown;
	float ViewportPosYOnButtonDown;

	/** Mouse position during the call on mouse button up. */
	FVector2D MousePositionOnButtonUp;

	bool bIsLMB_Pressed = false;
	bool bIsRMB_Pressed = false;

	/** True, if the user is currently interactively scrolling the view (ex.: by holding the left mouse button and dragging). */
	bool bIsScrolling = false;

	//////////////////////////////////////////////////
	// Misc

	FGeometry ThisGeometry;

	/** Cursor type. */
	ECursorType CursorType;
	
	FSimulationFrameView SimulationFrameView;

	FSimFrameHoverView HoverView;

	mutable float TooltipOpacity = 0.f;
	float PulseAccumulator=0;

	bool CompactSimFrameView() const { return bCompactSimFrameView; }
	bool LinearSimFrameView() const { return bLinearSimFrameView; }

	void ToggleCompactSimFrameView() { bCompactSimFrameView = !bCompactSimFrameView; bIsStateDirty = true;  }
	void ToggleLinearSimFrameview() { bLinearSimFrameView = !bLinearSimFrameView; bIsStateDirty = true; }

	bool bCompactSimFrameView = false;
	bool bLinearSimFrameView = true;

	FSimTime ViewportMaxSimTimeMS = 0;
	FSimTime ViewportMinSimTimeMS = 0;
	float ViewportYMaxValue = 0.f;
	float ViewportYContentStartValue = 30.f;

	FString UserStateSearchString;
	bool PerformSearch(const FSimulationData::FTick&, const FSimulationData::FRestrictedView& SimView);
	bool PerformSearch(const FSimulationData::FNetSerializeRecv&, const FSimulationData::FRestrictedView& SimView);
	bool PerformSearchInternal(const TArrayView<const FSimulationData::FUserState* const>& UserStates);

	void SortActorGroupToTop(const FSimulationActorGroup* Group);

	TArray<FSimNetActorID> UserSortedNetActors;

	bool IsFrameContentView_FrameNumber() const { return SimulationFrameView.ContentType == ESimFrameContentType::FrameNumber; }
	bool IsFrameContentView_BufferedInputCmds() const { return SimulationFrameView.ContentType == ESimFrameContentType::NumBufferedInputCmds; }

	void SetFrameContentView_FrameNumber() { SimulationFrameView.ContentType = ESimFrameContentType::FrameNumber; bIsStateDirty = true;}
	void SetFrameContentView_NumBufferdInputCmds() { SimulationFrameView.ContentType = ESimFrameContentType::NumBufferedInputCmds; bIsStateDirty = true; }
};
