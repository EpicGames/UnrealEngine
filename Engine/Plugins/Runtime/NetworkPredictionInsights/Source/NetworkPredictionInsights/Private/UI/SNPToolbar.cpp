// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNPToolbar.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "SNPWindow.h"
#include "NetworkPredictionInsightsManager.h"
#include "NetworkPredictionInsightsCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UICommandList.h"

#define LOCTEXT_NAMESPACE "NetworkPredictionInsightsModule"


SNPToolbar::SNPToolbar()
{
}

SNPToolbar::~SNPToolbar()
{
}

void SNPToolbar::Construct(const FArguments& InArgs, TSharedPtr<SNPWindow> InParentWindow)
{
	ParentWindow = InParentWindow;

	struct Local
	{
		static void FillViewToolbar(TSharedPtr<SNPWindow> ProfilerWindow, FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("View");
			{
				ToolbarBuilder.AddToolBarButton(FNetworkPredictionInsightsManager::GetCommands().ToggleAutoScrollSimulationFrames);
			}
			ToolbarBuilder.EndSection();
			/*
			ToolbarBuilder.BeginSection("Connection");
			{
				TSharedRef<SWidget> GameInstanceWidget = ProfilerWindow->CreateGameInstanceComboBox();
				ToolbarBuilder.AddWidget(GameInstanceWidget);

				TSharedRef<SWidget> ConnectionWidget = ProfilerWindow->CreateConnectionComboBox();
				ToolbarBuilder.AddWidget(ConnectionWidget);

				TSharedRef<SWidget> ConnectionModeWidget = ProfilerWindow->CreateConnectionModeComboBox();
				ToolbarBuilder.AddWidget(ConnectionModeWidget);
			}
			ToolbarBuilder.EndSection();
			ToolbarBuilder.BeginSection("New");
			{
				//TODO: ToolbarBuilder.AddToolBarButton(FNetworkPredictionManager::GetCommands().OpenNewWindow);
			}
			ToolbarBuilder.EndSection();
			*/
		}

		static void FillRightSideToolbarPre(FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("Debug");
			{
				ToolbarBuilder.AddToolBarButton(FNetworkPredictionInsightsManager::GetCommands().FirstEngineFrame);
				ToolbarBuilder.AddToolBarButton(FNetworkPredictionInsightsManager::GetCommands().PrevEngineFrame);
			}
			ToolbarBuilder.EndSection();
		}
		static void FillRightSideToolbarPost(FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("Debug");
			{
				ToolbarBuilder.AddToolBarButton(FNetworkPredictionInsightsManager::GetCommands().NextEngineFrame);
				ToolbarBuilder.AddToolBarButton(FNetworkPredictionInsightsManager::GetCommands().LastEngineFrame);
			}
			ToolbarBuilder.EndSection();
		}
	};
	

	TSharedPtr<FUICommandList> CommandList = ParentWindow->GetCommandList();

	FToolBarBuilder ToolbarBuilder(CommandList.ToSharedRef(), FMultiBoxCustomization::None);
	Local::FillViewToolbar(ParentWindow, ToolbarBuilder);


	FToolBarBuilder RightSideToolbarBuilderPrev(CommandList.ToSharedRef(), FMultiBoxCustomization::None);
	Local::FillRightSideToolbarPre(RightSideToolbarBuilderPrev);

	FToolBarBuilder RightSideToolbarBuilderPost(CommandList.ToSharedRef(), FMultiBoxCustomization::None);
	Local::FillRightSideToolbarPost(RightSideToolbarBuilderPost);

	// Create the tool bar
	ChildSlot
	[
		SNew(SHorizontalBox)
		
#if 0
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SComboButton)
			.Visibility(EVisibility::Visible)
			//.ComboButtonStyle(FSourceFilterStyle::Get(), "SourceFilter.ComboButton")
			//.ForegroundColor(FLinearColor::White)
			.ContentPadding(0.0f)
			.OnGetMenuContent(this, &SNPToolbar::OnGetOptionsMenu)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				/*
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					//.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
					//.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FText::FromString(FString(TEXT("\xf0b0"))))
				]*/
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 0, 0, 0)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					//.TextStyle(FSourceFilterStyle::Get(), "SourceFilter.TextStyle")
					.Text(LOCTEXT("OptionMenuLabel", "Options"))
				]
			]
		]
#endif

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)			
		.Padding(2.0f)
		.FillWidth(1.f)
		[
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.f)
			.Padding(2.0f)
			.MaxWidth(300.f)
			[
				SNew(SSearchBox)
				.HintText(LOCTEXT("UserStateSearchBoxHint", "Search simulation states"))
				.OnTextChanged(ParentWindow.ToSharedRef(), &SNPWindow::SearchUserData)
				.ToolTipText(LOCTEXT("UserStateSearchHint", "Type here to search for content in user simulation states"))
				.MinDesiredWidth(300.f)
			]
		

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBorder)
				.Padding(0)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				[
					ToolbarBuilder.MakeWidget()
				]
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f)
			[
				SAssignNew(PIESessionComboBox, SComboBox<TSharedPtr<int32>>)
				.ToolTipText(LOCTEXT("PIESessionComboBoxToolTip", "Filters view to a given session of PIE."))
				.OptionsSource(&ParentWindow->PIESessionOptions)
				.Visibility(MakeAttributeLambda([this] { return ParentWindow->PIESessionOptions.Num() > 1 ? EVisibility::Visible : EVisibility::Collapsed; }))
				.OnSelectionChanged(ParentWindow.ToSharedRef(), &SNPWindow::PIESessionComboBox_OnSelectionChanged)
				.OnGenerateWidget(ParentWindow.ToSharedRef(), &SNPWindow::PIESessionComboBox_OnGenerateWidget)
				[
					SNew(STextBlock)
					.Text(ParentWindow.ToSharedRef(), &SNPWindow::PIESessionComboBox_GetSelectionText)
				]
			]
		]
			

		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			
			SNew(SHorizontalBox)		
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f)
			[
				RightSideToolbarBuilderPrev.MakeWidget()
			]
		
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBorder)
				.Padding(0)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				[
					SNew(SNumericEntryBox<uint64>)
					.ToolTipText(LOCTEXT("EngineFrameToolTip", "Current Engine Frame the simulation timeline is viewing."))
					.OnValueCommitted_Lambda([this](uint64 InValue, ETextCommit::Type InCommitType){ ParentWindow->SetEngineFrame(InValue); })
					.Value_Lambda([this]() { return ParentWindow->GetCurrentEngineFrame(); } )
					.MinValue_Lambda([this]() { return ParentWindow->GetMinEngineFrame(); } )
					.MaxValue_Lambda([this]() { return ParentWindow->GetMaxEngineFrame(); } )
				]
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f)
			[
				RightSideToolbarBuilderPost.MakeWidget()
			]
		]
	];
}

TSharedRef<SWidget> SNPToolbar::OnGetOptionsMenu()
{
	FMenuBuilder Builder(true, TSharedPtr<FUICommandList>());
	if (ParentWindow.IsValid())
	{
		ParentWindow->OnGetOptionsMenu(Builder);
	}
	return Builder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
