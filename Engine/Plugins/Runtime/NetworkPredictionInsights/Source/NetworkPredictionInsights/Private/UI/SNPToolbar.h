// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SCompoundWidget.h"

template <typename OptionType> class SComboBox;

///////////////////////////////////////////////////////////////////////////////////////////////////
class SNPWindow;

class SNPToolbar : public SCompoundWidget
{
public:
	/** Default constructor. */
	SNPToolbar();

	/** Virtual destructor. */
	virtual ~SNPToolbar();

	SLATE_BEGIN_ARGS(SNPToolbar) {}
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<SNPWindow> InProfilerWindow);

private:
	TSharedPtr<SNPWindow> ParentWindow;

	TSharedRef<SWidget> OnGetOptionsMenu();

	TSharedPtr<SComboBox<TSharedPtr<int32>>> PIESessionComboBox;

};
