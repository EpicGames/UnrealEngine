// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNPWindow.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Text/STextBlock.h"

#include "SNPSimFrameContents.h"
#include "Modules/ModuleManager.h"
#include "Insights/IUnrealInsightsModule.h"
#include "NetworkPredictionInsightsManager.h"
#include "NetworkPredictionInsightsCommands.h"
#include "SNPSimFrameView.h"
#include "SNPToolbar.h"

#define LOCTEXT_NAMESPACE "SNetworkPredictionWindow"

const FName FNetworkPredictionInsightsTabs::ToolbarID("Toolbar");
const FName FNetworkPredictionInsightsTabs::SimFrameViewID("SimFrameView");
const FName FNetworkPredictionInsightsTabs::SimFrameContentsID("SimFrameContents");

SNPWindow::SNPWindow()
{

}

SNPWindow::~SNPWindow()
{

}

void SNPWindow::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	InsightsModule = &FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	CommandList = MakeShareable(new FUICommandList);

	// Create & initialize tab manager.
	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);
	TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("NetworkPredictionMenuGroupName", "Network Prediction Insights"));

	
	TabManager->RegisterTabSpawner(FNetworkPredictionInsightsTabs::ToolbarID, FOnSpawnTab::CreateRaw(this, &SNPWindow::SpawnTab_Toolbar))
		.SetDisplayName(LOCTEXT("DeviceToolbarTabTitle", "Toolbar"))
		//.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "Toolbar.Icon.Small"))
		.SetGroup(AppMenuGroup);
	
	TabManager->RegisterTabSpawner(FNetworkPredictionInsightsTabs::SimFrameViewID, FOnSpawnTab::CreateRaw(this, &SNPWindow::SpawnTab_SimFrameView))
		.SetDisplayName(LOCTEXT("NetworkPredictionInsights.SimFrameViewTabTitle", "Simulation Frame View"))
		//.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "PacketView.Icon.Small"))
		.SetGroup(AppMenuGroup);

	
	TabManager->RegisterTabSpawner(FNetworkPredictionInsightsTabs::SimFrameContentsID, FOnSpawnTab::CreateRaw(this, &SNPWindow::SpawnTab_SimFrameContentView))
		.SetDisplayName(LOCTEXT("NetworkPredictionInsights.SimFrameContentsTitle", "Simulation Frame Content"))
		//.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "PacketContentView.Icon.Small"))
		.SetGroup(AppMenuGroup);
	/*

	TabManager->RegisterTabSpawner(FNetworkingProfilerTabs::NetStatsViewID, FOnSpawnTab::CreateRaw(this, &SNetworkingProfilerWindow::SpawnTab_NetStatsView))
		.SetDisplayName(LOCTEXT("NetworkingProfiler.NetStatsViewTabTitle", "Net Stats"))
		.SetIcon(FSlateIcon(FInsightsStyle::GetStyleSetName(), "NetStatsView.Icon.Small"))
		.SetGroup(AppMenuGroup);
	*/
	
	//TSharedPtr<FNetworkPredictionManager> NetworkingProfilerManager = FNetworkPredictionManager::Get();
	//ensure(NetworkingProfilerManager.IsValid());

	// Create tab layout.
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("NetworkingPredictionInsightsProfilerLayout_v1.0")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			
			->Split
			(
				FTabManager::NewStack()
				->AddTab(FNetworkPredictionInsightsTabs::ToolbarID, ETabState::OpenedTab)
				->SetHideTabWell(true)
				->SetSizeCoefficient(0.10f)
			)
			
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(1.0f)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.65f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.65f)
						->SetHideTabWell(true)
						->AddTab(FNetworkPredictionInsightsTabs::SimFrameViewID, ETabState::OpenedTab)
					)
					
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.35f)
						->SetHideTabWell(true)
						->AddTab(FNetworkPredictionInsightsTabs::SimFrameContentsID, ETabState::OpenedTab)
					)
				)
				/*
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.35f)
					//->AddTab(FNetworkingProfilerTabs::NetStatsViewID, ETabState::OpenedTab)
					//->SetForegroundTab(FNetworkingProfilerTabs::NetStatsViewID)
				)
				*/
			)
		);

	// Create & initialize main menu.
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(CommandList);

	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("MenuLabel", "MENU"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(this, &SNPWindow::FillMenu, TabManager),
		FName(TEXT("MENU"))
	);

	TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();

	ChildSlot
		[
			SNew(SOverlay)

			// Version
			+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Top)
				.Padding(0.0f, -16.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
						.Clipping(EWidgetClipping::ClipToBoundsWithoutIntersecting)
						.Text(LOCTEXT("NetworkPredictionInsightsVersion", "0.1e-32"))
						.ColorAndOpacity(FLinearColor(0.15f, 0.15f, 0.15f, 1.0f))
				]

			// Overlay slot for the main window area
			+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							MenuWidget
						]

					+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
						]
				]

			// Session hint overlay
			/*
			+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
						//.Visibility(this, &SNetworkingProfilerWindow::IsSessionOverlayVisible)
						.BorderImage(FAppStyle::GetBrush("NotificationList.ItemBackground"))
						.Padding(8.0f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("SelectTraceOverlayText", "Please select a trace."))
						]
				]
			*/
		];

	// Tell tab-manager about the global menu bar.
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);

	BindCommands();

	SetEngineFrame(0);
}

void SNPWindow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsModule->GetAnalysisSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		if (const INetworkPredictionProvider* NetworkPredictionProvider = ReadNetworkPredictionProvider(*Session.Get()))
		{
			uint64 NewDataCounter = NetworkPredictionProvider->GetNetworkPredictionDataCounter();
			if (NewDataCounter != CachedProviderDataCounter)
			{
				PopulateFilteredDataView();
				CachedProviderDataCounter = NewDataCounter;
			}
		}
	}
}

void SNPWindow::Reset()
{

}

void SNPWindow::BindCommands()
{
	Map_ToggleAutoScrollSimulationFrames();

	CommandList->MapAction(
		FNetworkPredictionInsightsManager::GetCommands().NextEngineFrame,
		FExecuteAction::CreateSP(this, &SNPWindow::NextEngineFrame),
		FCanExecuteAction::CreateSP(this, &SNPWindow::CanNextEngineFrame),
		EUIActionRepeatMode::RepeatEnabled);

	CommandList->MapAction(
		FNetworkPredictionInsightsManager::GetCommands().PrevEngineFrame,
		FExecuteAction::CreateSP(this, &SNPWindow::PrevEngineFrame),
		FCanExecuteAction::CreateSP(this, &SNPWindow::CanPrevEngineFrame),
		EUIActionRepeatMode::RepeatEnabled);
	
	CommandList->MapAction(
		FNetworkPredictionInsightsManager::GetCommands().FirstEngineFrame,
		FExecuteAction::CreateSP(this, &SNPWindow::FirstEngineFrame),
		FCanExecuteAction::CreateSP(this, &SNPWindow::CanFirstEngineFrame));

	CommandList->MapAction(
		FNetworkPredictionInsightsManager::GetCommands().LastEngineFrame,
		FExecuteAction::CreateSP(this, &SNPWindow::LastEngineFrame),
		FCanExecuteAction::CreateSP(this, &SNPWindow::CanLastEngineFrame));
}

FReply SNPWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// This isn't doing anything and I cant figure out why
	if(CommandList->ProcessCommandBindings(InKeyEvent.GetKey(), InKeyEvent.GetModifierKeys(), InKeyEvent.IsRepeat()))
	{
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

const bool SNPWindow::AutoScrollSimulationFrames() const
{
	if (FrameView.IsValid())
	{
		return FrameView->GetAutoScroll();
	}

	return false;
}
void SNPWindow::SetAutoScrollSimulationFrames(const bool bAutoScroll)
{
	if (FrameView.IsValid())
	{
		FrameView->SetAutoScroll(bAutoScroll);
	}
}

void SNPWindow::SetAutoScrollDirty()
{
	if (FrameView.IsValid())
	{
		FrameView->SetAutoScrollDirty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Toggle Commands
////////////////////////////////////////////////////////////////////////////////////////////////////

#define IMPLEMENT_TOGGLE_COMMAND(CmdName, IsEnabled, SetEnabled) \
	\
	void SNPWindow::Map_##CmdName()\
	{\
		CommandList->MapAction(FNetworkPredictionInsightsManager::GetCommands().CmdName, CmdName##_Custom());\
	}\
	\
	const FUIAction SNPWindow::CmdName##_Custom() \
	{\
		FUIAction UIAction;\
		UIAction.ExecuteAction = FExecuteAction::CreateRaw(this, &SNPWindow::CmdName##_Execute);\
		UIAction.CanExecuteAction = FCanExecuteAction::CreateRaw(this, &SNPWindow::CmdName##_CanExecute);\
		UIAction.GetActionCheckState = FGetActionCheckState::CreateRaw(this, &SNPWindow::CmdName##_GetCheckState);\
		return UIAction;\
	}\
	\
	void SNPWindow::CmdName##_Execute()\
	{\
		SetEnabled(!IsEnabled());\
	}\
	\
	bool SNPWindow::CmdName##_CanExecute() const\
	{\
		return true;\
	}\
	\
	ECheckBoxState SNPWindow::CmdName##_GetCheckState() const\
	{\
		return IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;\
	}

IMPLEMENT_TOGGLE_COMMAND(ToggleAutoScrollSimulationFrames, AutoScrollSimulationFrames, SetAutoScrollSimulationFrames)
#undef IMPLEMENT_TOGGLE_COMMAND

void SNPWindow::FillMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> InTabManager)
{
	if (!InTabManager.IsValid())
	{
		return;
	}

#if !WITH_EDITOR
	//TODO: FGlobalTabmanager::Get()->PopulateTabSpawnerMenu(MenuBuilder, WorkspaceMenu::GetMenuStructure().GetStructureRoot());
#endif //!WITH_EDITOR

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);

	OnGetOptionsMenu(MenuBuilder);
}

TSharedRef<SDockTab> SNPWindow::SpawnTab_Toolbar(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(true)
		.TabRole(ETabRole::PanelTab)
		[
			SNew(SNPToolbar, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SNPWindow::OnToolbarTabClosed));
	return DockTab;
}

TSharedRef<SDockTab> SNPWindow::SpawnTab_SimFrameView(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(FrameView, SNPSimFrameView, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SNPWindow::OnSimFrameViewTabClosed));
	return DockTab;
}

TSharedRef<SDockTab> SNPWindow::SpawnTab_SimFrameContentView(const FSpawnTabArgs& Args)
{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SAssignNew(SimFrameContents, SNPSimFrameContents, SharedThis(this))
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &SNPWindow::OnSimFrameContentTabClosed));
	return DockTab;

}

void SNPWindow::OnSimFrameViewTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	FrameView = nullptr;
}

void SNPWindow::OnToolbarTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	
}

void SNPWindow::OnSimFrameContentTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	SimFrameContents = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////

void SNPWindow::SetEngineFrame(uint64 InFrame, const bool bSetAutoScrollDirty)
{
	MaxEngineFrameHistory.SetNum(EngineFrameHistoryIndex+1);
	MaxEngineFrameHistory.Add(InFrame);
	EngineFrameHistoryIndex = MaxEngineFrameHistory.Num()-1;
	
	if (MaxEngineFrameHistory.Num() > 32)
	{
		MaxEngineFrameHistory.RemoveAt(0, 16, EAllowShrinking::No);
		EngineFrameHistoryIndex -= 16;
		check(EngineFrameHistoryIndex >= 0);
	}

	Filter.MaxEngineFrame = InFrame;
	PopulateFilteredDataView();
	if (bSetAutoScrollDirty)
	{
		SetAutoScrollDirty();
	}
}
uint64 SNPWindow::GetCurrentEngineFrame() const
{
	return Filter.MaxEngineFrame == 0 ? UnfilteredDataView.LastEngineFrame : Filter.MaxEngineFrame;
}
uint64 SNPWindow::GetMinEngineFrame() const
{
	return UnfilteredDataView.FirstEngineFrame;

}
uint64 SNPWindow::GetMaxEngineFrame() const
{
	return UnfilteredDataView.LastEngineFrame;
}

void SNPWindow::NextEngineFrame()
{
	SetEngineFrame(FMath::Min<uint64>(GetCurrentEngineFrame()+1, GetMaxEngineFrame()));
}
bool SNPWindow::CanNextEngineFrame() const
{
	return (GetCurrentEngineFrame() < GetMaxEngineFrame());
}

void SNPWindow::PrevEngineFrame()
{
	// Don't get confused: the << < DVR > >>  is about setting MaxEngineFrame to visualize up to. We haven't implemented "engine frame window" yet.
	SetEngineFrame(FMath::Max<uint64>(GetCurrentEngineFrame()-1, GetMinEngineFrame()));
}
bool SNPWindow::CanPrevEngineFrame() const
{
	return (GetCurrentEngineFrame() > GetMinEngineFrame());
}

void SNPWindow::FirstEngineFrame()
{
	SetEngineFrame(GetMinEngineFrame());
	SetAutoScrollDirty();
}
bool SNPWindow::CanFirstEngineFrame() const
{
	return (GetCurrentEngineFrame() > GetMinEngineFrame());
}

void SNPWindow::LastEngineFrame()
{
	SetEngineFrame(0); // Ok, we are cheating. Set it to 0 to mean "uncapped" so that we'll start taking in live data again. This UI isn't great.
	SetAutoScrollSimulationFrames(true);
}
bool SNPWindow::CanLastEngineFrame() const
{
	return (GetCurrentEngineFrame() < GetMaxEngineFrame());
}

void SNPWindow::NotifySimContentClicked(const FSimContentsView& Content)
{
	SelectedContent = Content;
	if (SimFrameContents.IsValid())
	{
		SimFrameContents->NotifyContentClicked(SelectedContent);
	}
}

void SNPWindow::JumpPreviousViewedEngineFrame()
{
	EngineFrameHistoryIndex = FMath::Max(0, EngineFrameHistoryIndex - 1);
	
	Filter.MaxEngineFrame = MaxEngineFrameHistory[EngineFrameHistoryIndex];
	PopulateFilteredDataView();
	SetAutoScrollSimulationFrames(true);
}

void SNPWindow::JumpNextViewedEngineFrame()
{
	EngineFrameHistoryIndex = FMath::Min(MaxEngineFrameHistory.Num()-1, EngineFrameHistoryIndex + 1);
	
	Filter.MaxEngineFrame = MaxEngineFrameHistory[EngineFrameHistoryIndex];
	PopulateFilteredDataView();
	SetAutoScrollSimulationFrames(true);
}

void SNPWindow::SearchUserData(const FText& InFilterText)
{
	if (FrameView.IsValid())
	{
		FrameView->SearchUserData(InFilterText);
	}
}

void SNPWindow::PopulateFilteredDataView()
{
	FilteredDataCollection.Simulations.Reset();
	FilteredDataCollection.FirstEngineFrame = TNumericLimits<uint64>::Max();
	FilteredDataCollection.LastEngineFrame = 0;
	
	UnfilteredDataView.FirstEngineFrame = TNumericLimits<uint64>::Max();
	UnfilteredDataView.LastEngineFrame = 0;

	TSharedPtr<const TraceServices::IAnalysisSession> Session = InsightsModule->GetAnalysisSession();
	if (Session.IsValid())
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
		const INetworkPredictionProvider* NetworkPredictionProvider = ReadNetworkPredictionProvider(*Session.Get());
		if (NetworkPredictionProvider)
		{
			TArrayView<const TSharedRef<FSimulationData>> UnfilteredSimulationList = NetworkPredictionProvider->ReadSimulationData();
			if (UnfilteredSimulationList.Num() == 0)
			{
				return;
			}

			TArray<TSharedPtr<const FSimulationData>> FilteredSimulationList;
			FilteredSimulationList.Reserve(UnfilteredSimulationList.Num());

			// Build PIE Session list
			for (auto& SharedRef : UnfilteredSimulationList)
			{
				const int32 PIESession = SharedRef->ConstData.ID.PIESession;
				if (!PIESessionOptions.FindByPredicate([PIESession](const TSharedPtr<int32>& SharedPtr) { return *SharedPtr == PIESession; }))
				{
					PIESessionOptions.Add(MakeShareable(new int32(PIESession)));
					SetEngineFrame(0, true);
					SelectedPIESession = PIESession;
					SetAutoScrollSimulationFrames(true);
					
				}
			}

			check(PIESessionOptions.Num() > 0);
			const uint64 FilteredPieSession = SelectedPIESession == -1 ? *PIESessionOptions.Last() : SelectedPIESession;

			// Filter
			for (auto& SharedRef : UnfilteredSimulationList)
			{
				if (SharedRef->ConstData.ID.SimID <= 0)
				{
					// No NetGUID assigned yet (this would only happen first few frames in a live session)
					continue;
				}

				// TODO: actually filter by group name or actor class, etc

				if (FilteredPieSession != SharedRef->ConstData.ID.PIESession)
				{
					continue;
				}

				FilteredSimulationList.Add(SharedRef);
			}

			// Process Filtered Simulation List into restricted views on the engine frames we want to pass on
			for (auto& SharedPtr : FilteredSimulationList)
			{
				const FSimulationData* SimData = SharedPtr.Get();
				if (!ensure(SimData))
				{
					continue;
				}

				uint64 SimMinEngineFrame = TNumericLimits<uint64>::Max();
				uint64 SimMaxEngineFrame = 0;

				if (SimData->Ticks.Num() > 0)
				{
					SimMinEngineFrame = FMath::Min<uint64>(SimMinEngineFrame, SimData->Ticks[0].EngineFrame);
					SimMaxEngineFrame = FMath::Max<uint64>(SimMinEngineFrame, SimData->Ticks[SimData->Ticks.Num()-1].EngineFrame);
				}
				
				if (SimData->NetRecv.Num() > 0)
				{
					SimMinEngineFrame = FMath::Min<uint64>(SimMinEngineFrame, SimData->NetRecv[0].EngineFrame);
					SimMaxEngineFrame = FMath::Max<uint64>(SimMinEngineFrame, SimData->NetRecv[SimData->NetRecv.Num()-1].EngineFrame);
				}

				if (SimMaxEngineFrame < SimMinEngineFrame)
				{
					continue;
				}

				UnfilteredDataView.FirstEngineFrame = FMath::Min(UnfilteredDataView.FirstEngineFrame, SimMinEngineFrame);
				UnfilteredDataView.LastEngineFrame = FMath::Max(UnfilteredDataView.LastEngineFrame, SimMaxEngineFrame);

				if (SimMaxEngineFrame < Filter.MinEngineFrame)
				{
					continue;
				}

				if (Filter.MaxEngineFrame != 0 && SimMinEngineFrame > Filter.MaxEngineFrame)
				{
					continue;
				}

				// Find the valid ranges for the simulation data
				const uint64 FirstFilteredFrame = FMath::Max(SimMinEngineFrame, Filter.MinEngineFrame);
				const uint64 LastFilteredFrame = Filter.MaxEngineFrame == 0 ? SimMaxEngineFrame : FMath::Min(SimMaxEngineFrame, Filter.MaxEngineFrame);

				FilteredDataCollection.FirstEngineFrame = FMath::Min(FilteredDataCollection.FirstEngineFrame, FirstFilteredFrame);
				FilteredDataCollection.LastEngineFrame = FMath::Max(FilteredDataCollection.LastEngineFrame, LastFilteredFrame);

				FilteredDataCollection.Simulations.Emplace(SimData->MakeRestrictedView(Filter.MinEngineFrame, FilteredDataCollection.LastEngineFrame));
			}

		}
	}

	OnFilteredDataCollectionChange.Broadcast(FilteredDataCollection);
}

void SNPWindow::OnGetOptionsMenu(FMenuBuilder& Builder)
{
	if (FrameView.IsValid())
	{
		FrameView->OnGetOptionsMenu(Builder);
	}
}

FText SNPWindow::PIESessionComboBox_GetSelectionText() const
{
	uint64 ActualSession = SelectedPIESession;
	if (SelectedPIESession == -1 && PIESessionOptions.Num() > 0)
	{
		ActualSession = *PIESessionOptions.Last();
	}

	return FText::FromString( FString::Printf(TEXT("PIE Session: %" UINT64_FMT), ActualSession) );
}

void SNPWindow::PIESessionComboBox_OnSelectionChanged(TSharedPtr<int32> NewPIEMode, ESelectInfo::Type SelectInfo)
{
	if (NewPIEMode.IsValid())
	{
		SelectedPIESession = *NewPIEMode;
		PopulateFilteredDataView();
		SetEngineFrame(UnfilteredDataView.LastEngineFrame, true);
	}
}

TSharedRef<SWidget> SNPWindow::PIESessionComboBox_OnGenerateWidget(TSharedPtr<int32> InPIEMode) const
{
	return SNew(STextBlock)
		.Text(FText::FromString(LexToString(*InPIEMode)));
}

// ------------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
