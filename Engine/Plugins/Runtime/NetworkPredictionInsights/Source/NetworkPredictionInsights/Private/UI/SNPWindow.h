// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "INetworkPredictionProvider.h"

class FMenuBuilder;
class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class SDockTab;
enum class ECheckBoxState : uint8;
namespace ESelectInfo { enum Type : int; }
struct FUIAction;

namespace TraceServices
{
	class IAnalysisService;
	class IAnalysisSession;
}

class SNPSimFrameView;
class SNPSimFrameContents;
class SNPWindow;

struct FNetworkPredictionInsightsTabs
{
	// Tab identifiers
	static const FName ToolbarID;
	static const FName SimFrameViewID;
	static const FName SimFrameContentsID;
};

// --------------------------------------------------------------------------------------------------------------------------------------------
//	Notes on the forms that our data takes
//
//	[Runtime] Runtime data is traced via UE_NP_TRACE macros
//	[TraceStream] data is now in some generic trace binary format (saved to file or sent over TCP connection)

// <----- Standalone UnrealInsights or in Editor. No longer dependent on NetworkPrediction module! ---->

//	[Analyzed] FNetworkPredictionAnalyzer processes data from trace stream and pushes it to provider 
//	[Provided] FNetworkPredictionProvider stores "all the data" ready for someone to consume it

//	<---- UI Widgets ------>

//	[Filtered] We want to limit the range or type of data that the UI should display
//		-Filter frame range
//		-Filter type of sim (movement, parametric, etc)
//		-Filter by network role (e.g, exclude simulated proxy)
//
//	[Sampled] We may need to merge some data, e.g., > 1 sim frame per pixel requires us to combine data to be rendered by UI (e.g, zooming out)	// TODO
//	[Culled] We still need to cull out data that is off screen, based on viewport state. We don't want to do this in ::OnPaint since that is called every render frame.
//
//	<----- Pixels on your screen! ---->
// --------------------------------------------------------------------------------------------------------------------------------------------

struct FNPFilter
{
	uint64 MinEngineFrame=0; // Inclusive (0 = "start at frame 0")
	uint64 MaxEngineFrame=0; // 0 = open ended. Otherwise, exclusive (1 = "only frame 0")
};

// Some cached values about our data that is not filtered
struct FNPUnfilteredDataView
{
	uint64 FirstEngineFrame = 0; // First Engine frame that had NP data
	uint64 LastEngineFrame = 0; // Last engine frame that had NP data
};


// -------------------------------------------------------------------------------------------------
//
// -------------------------------------------------------------------------------------------------

struct FFilter
{
	uint64 MinEngineFrame=0; // Inclusive (0 = "start at frame 0")
	uint64 MaxEngineFrame=0; // 0 = open ended. Otherwise, exclusive (1 = "only frame 0")
};

// This is the main data structure that is passed into the various widgets. Widgets should avoid reading directly from the provider API
struct FFilteredDataCollection
{
	TArray<TSharedRef<FSimulationData::FRestrictedView>> Simulations;

	uint64 FirstEngineFrame = 0;
	uint64 LastEngineFrame = 0;
};

// View into a specific piece of simulation content. E.g, something you click on in the sim timeline view.
struct FSimContentsView
{
	TSharedPtr<FSimulationData::FRestrictedView> SimView;

	const FSimulationData::FTick* SimTick = nullptr;
	const FSimulationData::FNetSerializeRecv* NetRecv = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

// Network Prediction Window, this is the top level / starting point for the NP Insights UI
class SNPWindow : public SCompoundWidget
{

public:

	SNPWindow();
	virtual ~SNPWindow();

	SLATE_BEGIN_ARGS(SNPWindow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void Reset();

	const TSharedPtr<FUICommandList> GetCommandList() const { return CommandList; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

#define DECLARE_TOGGLE_COMMAND(CmdName)\
public:\
	void Map_##CmdName(); /**< Maps UI command info CmdName with the specified UI command list. */\
	const FUIAction CmdName##_Custom(); /**< UI action for CmdName command. */\
private:\
	void CmdName##_Execute(); /**< Handles FExecuteAction for CmdName. */\
	bool CmdName##_CanExecute() const; /**< Handles FCanExecuteAction for CmdName. */\
	ECheckBoxState CmdName##_GetCheckState() const; /**< Handles FGetActionCheckState for CmdName. */

	DECLARE_TOGGLE_COMMAND(ToggleAutoScrollSimulationFrames)
#undef DECLARE_TOGGLE_COMMAND

public:

	const bool AutoScrollSimulationFrames() const;
	void SetAutoScrollSimulationFrames(const bool bAutoScroll);
	void SetAutoScrollDirty(); // Forces one auto scroll but does not change user-selectable state

	void SetEngineFrame(uint64 InFrame, const bool bSetAutoScrollDirty=false);
	uint64 GetCurrentEngineFrame() const;
	uint64 GetMinEngineFrame() const;
	uint64 GetMaxEngineFrame() const;

	void NextEngineFrame();
	bool CanNextEngineFrame() const;

	void PrevEngineFrame();
	bool CanPrevEngineFrame() const;

	void FirstEngineFrame();
	bool CanFirstEngineFrame() const;

	void LastEngineFrame();
	bool CanLastEngineFrame() const;

	void JumpPreviousViewedEngineFrame();
	void JumpNextViewedEngineFrame();

	// Called when the filtered view of our data has changed.
	// Note this won't be called just because new data (within our filter) has been added by the provider
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFilteredDataCollectionChange, const FFilteredDataCollection&)
	FOnFilteredDataCollectionChange OnFilteredDataCollectionChange;

	const FFilteredDataCollection& GetFilteredDataCollection() const { return FilteredDataCollection; }
	
	void OnGetOptionsMenu(FMenuBuilder& Builder);

	// Notifies from SimFrame Timeline View
	void NotifySimContentClicked(const FSimContentsView& Content);

	const FSimContentsView& GetSelectedContent() const { return SelectedContent; }

	void SearchUserData(const FText& InFilterText);


	FText PIESessionComboBox_GetSelectionText() const;
	void PIESessionComboBox_OnSelectionChanged(TSharedPtr<int32> NewPIEMode, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> PIESessionComboBox_OnGenerateWidget(TSharedPtr<int32> InPIEMode) const;

	TArray<TSharedPtr<int32>> PIESessionOptions;
	int32 SelectedPIESession = -1;

private:

	void PopulateFilteredDataView();

	FNPFilter Filter;
	FNPUnfilteredDataView UnfilteredDataView;

	FFilteredDataCollection FilteredDataCollection;

	void FillMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager);

	void BindCommands();	

	/** Commandlist used in the window (Maps commands to window specific actions) */
	TSharedPtr<FUICommandList> CommandList;

	/** Holds the tab manager that manages the front-end's tabs. */
	TSharedPtr<FTabManager> TabManager;

	TSharedRef<SDockTab> SpawnTab_Toolbar(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SimFrameView(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SimFrameContentView(const FSpawnTabArgs& Args);
	void OnToolbarTabClosed(TSharedRef<SDockTab> TabBeingClosed);
	void OnSimFrameViewTabClosed(TSharedRef<SDockTab> TabBeingClosed);
	void OnSimFrameContentTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	TSharedPtr<SNPSimFrameView> FrameView;
	TSharedPtr<SNPSimFrameContents> SimFrameContents;

	class IUnrealInsightsModule* InsightsModule;

	FSimContentsView SelectedContent;
	
	uint64 CachedProviderDataCounter = 0;
	
	TArray<uint64> MaxEngineFrameHistory;
	int32 EngineFrameHistoryIndex = -1;
};
