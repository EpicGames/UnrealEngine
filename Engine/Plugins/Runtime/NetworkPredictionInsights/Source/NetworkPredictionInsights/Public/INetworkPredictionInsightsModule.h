// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace NetworkPredictionInsightsTabs
{
	extern NETWORKPREDICTIONINSIGHTS_API const FName DocumentTab;
};
