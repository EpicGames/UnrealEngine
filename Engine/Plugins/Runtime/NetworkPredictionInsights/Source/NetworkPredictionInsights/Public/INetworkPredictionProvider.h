// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/AnalysisSession.h"
#include "Common/PagedArray.h"

// In the NetworkPrediction system, simulations can individually specify their types used for timing.
// Since we can't really specialize the insights side per simulation, we are just using single typedefs here that
// could be compiled to larger target if necessary.

#ifndef NP_INSIGHTS_SIM_TIME
#define NP_INSIGHTS_SIM_TIME int32
#endif

using FSimTime = NP_INSIGHTS_SIM_TIME;

// Mirroring this is unavoidable in order to keep NetworkPredictionInsights from depending on EngineTypes
enum class ENP_NetRole: uint8
{
	/** No role at all. */
	None,
	/** Locally simulated proxy of this actor. */
	SimulatedProxy,
	/** Locally autonomous proxy of this actor. */
	AutonomousProxy,
	/** Authoritative control over the actor. */
	Authority,
	MAX,
};

// Must be kept in sync with ENetworkPredictionTickingPolicy
enum class ENP_TickingPolicy: uint8
{
	Independent = 1 << 0,
	Fixed = 1 << 1
};

// Must be kept in sync with ENetworkLOD
enum class ENP_NetworkLOD: uint8
{
	Interpolated,
	SimExtrapolate,
	ForwardPredict
};

enum class ENP_UserState: uint8
{
	Input,
	Sync,
	Aux,
	Physics,
	MAX
};

enum class ENP_UserStateSource: uint8
{
	Unknown			= 1 << 0,	// not set yet
	ProduceInput	= 1 << 1,	// System called produce input to sample new input locally (should only apply to InputCmd state)
	SynthInput		= 1 << 2,	// System created input synthetically: based on previous state or some guess.
	SimTick			= 1 << 3,	// Generated during a simulation tick
	NetRecv			= 1 << 4,	// Received from network but not committed to buffers
	NetRecvCommit	= 1 << 5,	// Received from network and committed to the buffers
	OOB				= 1 << 6	// Out of band modification (e.g, game code)
};

enum class ENetSerializeRecvStatus: uint8
{
	Unknown,	// The moment we trace a Net Recv, its status is unknown.
	
	Confirm,	//	Valid data that did not cause a correction.
	Rollback,	//	Caused simulation to rollback. A correction.
	Jump,		//	Caused simulation to jump forward, effectively reseting its state
	Fault,		//	Unable to process the data. For example, received frame we no longer had in our local buffers
	Stale,		//	Received multiple bunches in a frame, this was effectively replaced by a newer one and not processed.
};

const TCHAR* LexToString(ENP_NetRole Role);
const TCHAR* LexToString(ENP_UserState State);
const TCHAR* LexToString(ENP_UserStateSource Source);
const TCHAR* LexToString(ENetSerializeRecvStatus Status);
const TCHAR* LexToString(ENP_TickingPolicy Policy);

// How we identify actors across network connection and PIE sessions
struct FSimNetActorID
{
	int32 SimID;
	int32 PIESession;

	bool operator==(const FSimNetActorID& Other) const
	{
		return SimID == Other.SimID && PIESession == Other.PIESession;
	}
};


// ----------------------------------------------------------------------------------------------
//
// ----------------------------------------------------------------------------------------------

template<typename T>
struct TRestrictedPageArrayView;

template<typename T>
struct TRestrictedPageViewIterator
{
	TRestrictedPageViewIterator(const TRestrictedPageArrayView<T>& InPageArrayView, uint64 StartItemIndex=0)
		: PageArrayView(InPageArrayView), CurrentItemIdx(StartItemIndex)
	{
		It = PageArrayView.PageArray.GetIteratorFromItem(StartItemIndex);
	}

	void operator++() { CurrentItemIdx++; It++;}
	void operator--() { CurrentItemIdx--; It--;}
	operator bool()	{ return Valid(); }

	const T& operator* () const { if (Valid()) return *It.GetCurrentItem(); else { check(false); return *It.GetCurrentItem(); } }
	const T* operator->() const { return Valid() ? It.GetCurrentItem() : nullptr; }	

private:

	bool Valid() const { return CurrentItemIdx >= PageArrayView.StartItemIdx && CurrentItemIdx < PageArrayView.EndItemIdx; }

	const TRestrictedPageArrayView<T>& PageArrayView;
	mutable TraceServices::TPagedArrayIterator<T, typename TraceServices::TPagedArray<T>::PageType> It;
	uint64 CurrentItemIdx; // Kind of unfortunate but we store this ourselves to avoid calculating it off the iter each time
};

template<typename T>
struct TRestrictedPageArrayView
{
	TRestrictedPageArrayView(const TraceServices::TPagedArray<T>& InPageArray, uint64 InStart, uint64 InEnd)
		: PageArray(InPageArray), StartItemIdx(InStart), EndItemIdx(InEnd) { }

	TRestrictedPageViewIterator<T> GetIterator() const
	{
		return TRestrictedPageViewIterator<T>(*this, StartItemIdx);
	}

	TRestrictedPageViewIterator<T> GetIteratorFromEnd() const
	{
		return TRestrictedPageViewIterator<T>(*this, EndItemIdx > 0 ? EndItemIdx-1 : 0);
	}

	const T& GetLast() const
	{
		return PageArray[EndItemIdx-1];
	}

	const T& GetFirst() const
	{
		return PageArray[StartItemIdx];
	}

	const T& operator[](uint64 Index) const
	{
		check(Index >= StartItemIdx && Index < EndItemIdx);
		return PageArray[Index];
	}

	int32 GetNum() const
	{
		return EndItemIdx - StartItemIdx;
	}

private:

	template<typename>
	friend struct TRestrictedPageViewIterator;

	const TraceServices::TPagedArray<T>& PageArray;
	uint64 StartItemIdx;
	uint64 EndItemIdx;
};

template<typename T>
typename TEnableIf<!TIsPointer<T>::Value, uint64>::Type GetEngineFrame(const T& Element)
{
	return Element.EngineFrame;
}

template<typename T>
typename TEnableIf<TIsPointer<T>::Value, uint64>::Type GetEngineFrame(const T& Element)
{
	return Element->EngineFrame;
}


// Searches through PageArray for index of first element >= MinFrameNumber
template<typename T>
uint64 FindMinIndexPagedArray(const TraceServices::TPagedArray<T>& PageArray, uint64 MinFrameNumber)
{
	// Flip through pages first before iterating individual items
	for (uint64 PageIdx=0; PageIdx < PageArray.NumPages(); ++PageIdx)
	{
		auto* Page = const_cast<TraceServices::TPagedArray<T>&>(PageArray).GetPage(PageIdx);
		check(Page->Count >0);

		//if (Page->Items[Page->Count-1].EngineFrame >= MinFrameNumber)
		if (GetEngineFrame(Page->Items[Page->Count-1]) >= MinFrameNumber)
		{
			for (uint64 ItemIdx=0; ItemIdx < Page->Count; ++ItemIdx)
			{
				if (GetEngineFrame(Page->Items[ItemIdx]) >= MinFrameNumber)
				{
					return (PageIdx * PageArray.GetPageSize()) + ItemIdx;
				}
			}
		}
	}

	return 0;
}

// Searches through PageArray for index+1 of first element > MinFrameNumber. E.g, index to iterate up to (<) for valid EngineFrameNumber.
// Fixme: it probably makes more sense to just give this a start index and pickup where FindMinIndex left off
template<typename T>
uint64 FindMaxIndexPagedArray(const TraceServices::TPagedArray<T>& PageArray, uint64 MaxFrameNumber)
{
	if (PageArray.Num() == 0)
	{
		return 0;
	}
	
	if (MaxFrameNumber == 0)
	{
		return PageArray.Num();
	}

	// Flip through pages first before iterating individual items
	for (uint64 PageIdx=PageArray.NumPages()-1; ; --PageIdx)
	{
		auto* Page = const_cast<TraceServices::TPagedArray<T>&>(PageArray).GetPage(PageIdx);
		check(Page->Count >0);

		if (GetEngineFrame(Page->Items[0]) <= MaxFrameNumber)
		{
			for (uint64 ItemIdx=Page->Count-1; ; --ItemIdx)
			{
				if (GetEngineFrame(Page->Items[ItemIdx]) <= MaxFrameNumber)
				{
					return (PageIdx * PageArray.GetPageSize()) + ItemIdx + 1;
				}

				if (ItemIdx == 0)
					break;

			}
		}

		if (PageIdx == 0)
			break;
	}
	return 0;
}

template<typename T>
int32 FindMinIndexTArray(const TArray<const T*>& Array, uint64 MinFrameNumber)
{
	for (int32 idx=0; idx < Array.Num(); ++idx)
	{
		if (Array[idx]->EngineFrame >= MinFrameNumber)
		{
			return idx;
		}
	}

	return 0;
}

template<typename T>
int32 FindMaxIndexTArray(const TArray<const T*>& Array, uint64 MaxFrameNumber, int32 StartIdx=0)
{
	if (MaxFrameNumber == 0)
	{
		return Array.Num();
	}

	for (int32 idx=StartIdx; idx < Array.Num(); ++idx)
	{
		if (Array[idx]->EngineFrame > MaxFrameNumber)
		{
			return FMath::Max(0, idx);
		}
	}
	return Array.Num();
}

template<typename T>
struct TSparseFrameData
{
	const TSharedRef<const T> Read(uint64 EngineFrame) const
	{
		for (int32 idx=SparseData.Num()-1; idx >=0; --idx)
		{
			if (SparseData[idx]->EngineFrame <= EngineFrame)
			{
				return SparseData[idx];
			}
		}

		// Didn't write anything for this frame so we insert default data
		const_cast<TSparseFrameData<T>*>(this)->SparseData.Emplace(MakeShareable(new T(EngineFrame)));
		return SparseData.Last();
	}

	const TSharedRef<T>& Write(uint64 EngineFrame)
	{
		for (int32 idx=SparseData.Num()-1; idx >=0; --idx)
		{
			if (SparseData[idx]->EngineFrame == EngineFrame)
			{
				// Already exists
				return SparseData[idx];
			}

			if (SparseData[idx]->EngineFrame < EngineFrame)
			{
				// Must insert new
				break;
			}
		}

		SparseData.Emplace(MakeShareable(new T(EngineFrame)));
		return SparseData.Last();
	}

private:

	TArray<TSharedRef<T>> SparseData;
};

// Holds all data we traced for a given simulation.
struct FSimulationData
{
	FSimulationData(int32 InTraceID, TraceServices::ILinearAllocator& Allocator)
		: TraceID(InTraceID)
		, Ticks(Allocator, 1024)
		, NetRecv(Allocator, 1024)
		, UserData(Allocator)
	{ }

	struct FTick;

	struct FSystemFault
	{
		const TCHAR* Str; // Traced Fmt string of fault
	};

	struct FNetSerializeRecv
	{
		// Traced Data:
		uint64 EngineFrame;
		FSimTime SimTimeMS;
		int32 Frame;

		// Analysis Data:
		ENetSerializeRecvStatus Status = ENetSerializeRecvStatus::Unknown;
		bool bOrphan = true; // Hasn't been attached to FTick::StartNetRecv

		FSimulationData::FTick* NextTick = nullptr;
		TArray<FSystemFault> SystemFaults; // System faults encountered during recv
	};

	// Data for a single simulation tick
	struct FTick
	{
		// Traced Data:
		uint64 EngineFrame;

		FSimTime StartMS;
		FSimTime EndMS;

		int32 OutputFrame;
		int32 LocalOffsetFrame;
		int32 NumBufferedInputCmds;
		bool bInputFault;

		// Analysis Data:

		// Net Recv that was processed prior to this tick. Will often be null. Used to ensure the ticks are drawn on the right sub track (Y)
		const FNetSerializeRecv* StartNetRecv = nullptr;

		uint64 ConfirmedEngineFrame = 0;	// Engine frame this became confirmed (we serialized a time past this)
		uint64 TrashedEngineFrame = 0;		// Engine frame this became trashed (we resimulated this time in a later frame)
		bool bRepredict = false;			// This tick was a repredict: we had already simulated up to this point before

		bool bReconcileDueToOffsetChange = false;
		const TCHAR* ReconcileStr = nullptr;
	};

	// Data that changes rarely over time about the simulation
	struct FSparse
	{
		FSparse(uint64 InEngineFrame)
			: EngineFrame(InEngineFrame)
		{}

		uint64 EngineFrame;	// engine frame this data became valid
		ENP_NetRole NetRole = ENP_NetRole::None;
		bool bHasNetConnection = false;
		ENP_TickingPolicy TickingPolicy = ENP_TickingPolicy::Independent;
		ENP_NetworkLOD NetworkLOD = ENP_NetworkLOD::Interpolated;
		int32 ServiceMask = 0;
	};
	
	// Data that never changes about the simulation
	struct FConst
	{
		FSimNetActorID ID;
		FName GroupName;
		FString DebugName;
	};

	struct FUserState
	{
		uint64 EngineFrame;	// EngineFrame this was written
		int32 SimFrame; // This states sim frame (what frame# it was inserted into the buffer at)
		FUserState* Prev = nullptr; // Previously written user state at this SimFrame number. Prev->EngineFrame < this->EngineFrame.
		const TCHAR* UserStr = nullptr; // stable pointer to traced string representation of user state
		ENP_UserStateSource Source = ENP_UserStateSource::Unknown; // Analyzed source of this user state
		const TCHAR* OOBStr = nullptr; // if OOB mod, this is the user traced string (telling us who did it)
	};

	struct FUserStateStore
	{
		FUserStateStore(TraceServices::ILinearAllocator& Allocator)
			: UserStates(Allocator, 1024) { }
		
		FORCENOINLINE FUserState& Push(int32 Frame, uint64 EngineFrame)
		{
			// Make new state
			FUserState& NewElement = UserStates.PushBack();
			NewElement.EngineFrame = EngineFrame;
			NewElement.SimFrame = Frame;

			// Put it in FrameMap
			FUserState*& Head = FrameMap.FindOrAdd(Frame);
			if (Head)
			{
				ensure(Head->EngineFrame <= EngineFrame);
				NewElement.Prev = Head;
			}
			Head = &NewElement;

			// Mark Frame as valid
			const int32 Delta = (Frame+1) - PopulatedFrames.Num();
			if (Delta > 0)
			{
				PopulatedFrames.Add(false, Delta);
			}

			PopulatedFrames[Frame] = true;
			return NewElement;
		}

		FORCENOINLINE const FUserState* Get(int32 InFrame, uint64 MaxEngineFrame, uint8 ExcludeSourceMask, bool bSparseAccess) const
		{
			auto FindMatchAtFrame = [&](const int32 Frame)
			{
				// return first match at this Frame
				FUserState* State = FrameMap.FindRef(Frame);
				while (State)
				{
					if (State->EngineFrame <= MaxEngineFrame && (((uint8)State->Source & ExcludeSourceMask) == 0))
					{
						break;
					}

					State = State->Prev;
				};

				return State;
			};

			if (bSparseAccess)
			{
				if (FUserState* FoundMatch = FindMatchAtFrame(InFrame))
				{
					return FoundMatch;
				}

				// Reverse iterating through set bits not as easy as hoped but still beats continuous hash lookups
				for (TBitArray<>::FConstReverseIterator BitIt(PopulatedFrames); BitIt; ++BitIt)
				{
					if (BitIt.GetIndex() < InFrame && BitIt.GetValue())
					{
						if (FUserState* FoundMatch = FindMatchAtFrame(BitIt.GetIndex()))
						{
							return FoundMatch;
						}
					}
				}

				return nullptr;
			}

			return FindMatchAtFrame(InFrame);
		}
	
	private:

		// Notes on data storage here:
		//	-We may not have an FUserState for all Simulation Frames (interpolation case will be very sparse)
		//	-We may have an FUserState for each simulation frame, or even multiple (autoproxy/correction heavy cases)
		//	-User states should be pushed in ascending EngineFrame order but NOT necessarily in accessing SimulationFram 
		//	-We don't need to hyper optimize this but we should avoid linear searching 

		TraceServices::TPagedArray<FUserState> UserStates;	// Holds the actual FUserState. Pointers to these elements are stable
		TMap<int32, FUserState*> FrameMap;	// Acceleration map to find head FUserState for given simulation frame
		TBitArray<> PopulatedFrames;		// Bit array for which simulation frames are populated. Accelerates finding "last valid" state in sparse cases.
	};

	struct FUserData
	{
		FUserData(TraceServices::ILinearAllocator& Allocator)
			: Store{Allocator, Allocator, Allocator, Allocator}
		{ }

		FUserStateStore Store[(int32)ENP_UserState::MAX];
	};

	struct FRestrictedUserStateView
	{
		FRestrictedUserStateView(const FUserData& InUserData, uint64 InMaxEngineFrame)
			: UserData(InUserData), MaxEngineFrame(InMaxEngineFrame)
		{ }

		FORCENOINLINE const FUserState* Get(ENP_UserState Type, int32 SimFrame, uint64 InEngineFrame, uint8 ExcludeMask) const
		{
			const uint64 EngineFrame = FMath::Min(InEngineFrame, MaxEngineFrame);
			const bool bSparseAccess = (Type == ENP_UserState::Aux);
			return UserData.Store[(int32)Type].Get(SimFrame, EngineFrame, ExcludeMask, bSparseAccess);
		}

	private:

		const FUserData& UserData;
		uint64 MaxEngineFrame;
	};

	// -----------------------------------------------------------

	struct FRestrictedView
	{
		const TRestrictedPageArrayView<FTick> Ticks;
		const TRestrictedPageArrayView<FNetSerializeRecv> NetRecv;
		const FRestrictedUserStateView UserData;
		const TSharedRef<const FSparse> SparseData;
		const FConst& ConstData;
		const int32 TraceID;

		int32 GetMaxSimTime() const
		{
			int32 MaxSimTime = 0;
			if (Ticks.GetNum() > 0)
			{
				MaxSimTime = Ticks.GetLast().EndMS;
			}
			if (NetRecv.GetNum() > 0)
			{
				MaxSimTime = FMath::Max<int32>(MaxSimTime, NetRecv.GetLast().SimTimeMS);
			}
			return MaxSimTime;
		}

		uint64 GetMaxEngineFrame() const
		{
			uint64 MaxEngineFrame = 0;
			if (Ticks.GetNum() > 0)
			{
				MaxEngineFrame = Ticks.GetLast().EngineFrame;
			}
			if (NetRecv.GetNum() > 0)
			{
				MaxEngineFrame = FMath::Max<int32>(MaxEngineFrame, NetRecv.GetLast().EngineFrame);
			}
			return MaxEngineFrame;
		}
	};

	TSharedRef<FRestrictedView> MakeRestrictedView(uint64 MinEngineFrame, uint64 MaxEngineFrame) const
	{
		ensure(MaxEngineFrame != 0); // by now, "0 = uncapped" should be out of the picture

		return MakeShareable(new FRestrictedView  {
			{ Ticks, FindMinIndexPagedArray(Ticks, MinEngineFrame), FindMaxIndexPagedArray(Ticks, MaxEngineFrame) },
			{ NetRecv, FindMinIndexPagedArray(NetRecv, MinEngineFrame), FindMaxIndexPagedArray(NetRecv, MaxEngineFrame) },
			FRestrictedUserStateView(UserData, MaxEngineFrame),
			SparseData.Read(MaxEngineFrame), ConstData, TraceID
		});
	}

	// -----------------------------------------------------------

	int32 TraceID;
	
	TraceServices::TPagedArray<FTick> Ticks;
	TraceServices::TPagedArray<FNetSerializeRecv> NetRecv;	// Actually holds the allocated recv records
	
	TSparseFrameData<FSparse> SparseData;
	FConst ConstData;

	FUserData UserData;

	// Data that is only used as a temp/scratch pad during the analysis page. This should not show up in the UI or FRestrictedView
	struct FAnalysis
	{
		// NetRecvs that we haven't matched with a tick or declared orphaned yet
		TArray<FNetSerializeRecv*> PendingNetSerializeRecv;

		uint64 NetRecvItemIdx = 0;
		uint64 TrashItemIdx = 0;

		FSimTime MaxTickSimTimeMS = 0; // Highest EndMS we have seen so far from Tick traces

		int32 NumBufferedInputCmds = 0;
		bool bInputFault = false;

		TArray<FUserState*> PendingCommitUserStates; // NetRecv'd state that hasn't been commited
		TArray<FSystemFault> PendingSystemFaults;
		const TCHAR* PendingOOBStr = nullptr;
		const TCHAR* PendingReconcileStr = nullptr;

		int32 LocalFrameOffset = 0;
		bool bLocalFrameOffsetChanged = false;

	} Analysis;
};

class INetworkPredictionProvider : public TraceServices::IProvider
{
public:

	// Return the version reported in the trace
	// A return value of 0 indicates no NetworkPrediction trace data
	virtual uint32 GetNetworkPredictionTraceVersion() const = 0;

	// Returns generic counter that increments everytime we analyse network prediction related data.
	virtual uint64 GetNetworkPredictionDataCounter() const = 0;	
	
	// Returns all simulation data that has been analyzed
	virtual TArrayView<const TSharedRef<FSimulationData>> ReadSimulationData() const = 0;
};

NETWORKPREDICTIONINSIGHTS_API const INetworkPredictionProvider* ReadNetworkPredictionProvider(const TraceServices::IAnalysisSession& Session);
